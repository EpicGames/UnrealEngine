// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureCompressorModule.h"
#include "Math/RandomStream.h"
#include "ChildTextureFormat.h"
#include "Containers/IndirectArray.h"
#include "Hash/xxhash.h"
#include "ImageCoreUtils.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Async/ParallelFor.h"
#include "Modules/ModuleManager.h"
#include "Engine/TextureDefines.h"
#include "TextureFormatManager.h"
#include "TextureBuildUtilities.h"
#include "Interfaces/ITextureFormat.h"
#include "Misc/Paths.h"
#include "Tasks/Task.h"
#include "OpenColorIOWrapper.h"
#include "OpenColorIOWrapperModule.h"
#include "ImageCore.h"
#include "ImageParallelFor.h"
#include <cmath>

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

#if 0
// for saving image dumps:
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#endif

static TAutoConsoleVariable<int32> CVarDetailedMipAlphaLogging(
	TEXT("r.DetailedMipAlphaLogging"),
	0,
	TEXT("Prints extra log messages for tracking when alpha gets removed/introduced during texture")
	TEXT("image processing.")
);

DEFINE_LOG_CATEGORY_STATIC(LogTextureCompressor, Log, All);

/*------------------------------------------------------------------------------
	Mip-Map Generation
------------------------------------------------------------------------------*/

// NOTE: mip gen wrap/clamp does NOT correspond to Texture Address Wrap/Clamp setting !!
//		it comes from bPreserveBorder !
enum EMipGenAddressMode
{
	MGTAM_Wrap,
	MGTAM_Clamp,
	MGTAM_BorderBlack,
};

/**
 * 2D view into one slice of an image.
 */
struct FImageView2D
{
	/** Pointer to colors in the slice. */
	FLinearColor* SliceColors;
	/** Width of the slice. */
	int64 SizeX;
	/** Height of the slice. */
	int64 SizeY;

	FImageView2D() : SliceColors(nullptr), SizeX(0), SizeY(0) {}

	/** Initialization constructor. */
	FImageView2D(FImage& Image, int32 SliceIndex)
	{
		SizeX = Image.SizeX;
		SizeY = Image.SizeY;
		check( SliceIndex < Image.NumSlices );
		SliceColors = (&Image.AsRGBA32F()[0]) + SliceIndex * SizeY * SizeX;
	}

	/** Access a single texel. */
	FLinearColor& Access(int32 X, int32 Y)
	{
		return SliceColors[X + Y * SizeX];
	}

	/** Const access to a single texel. */
	const FLinearColor& Access(int32 X, int32 Y) const
	{
		return SliceColors[X + Y * SizeX];
	}

	bool IsValid() const { return SliceColors != nullptr; }

	static const FImageView2D ConstructConst(const FImage& Image, int32 SliceIndex)
	{
		return FImageView2D(const_cast<FImage&>(Image), SliceIndex);
	}

};

// 2D sample lookup with input conversion
// requires SourceImageData.SizeX and SourceImageData.SizeY to be power of two
template <EMipGenAddressMode AddressMode>
static const FLinearColor& LookupSourceMip(const FImageView2D& SourceImageData, int32 X, int32 Y)
{
	if(AddressMode == MGTAM_Wrap)
	{
		// wrap
		// ! requires pow2 sizes
		checkSlow( FMath::IsPowerOfTwo(SourceImageData.SizeX) );
		checkSlow( FMath::IsPowerOfTwo(SourceImageData.SizeY) );

		X = (int32)((uint32)X) & (SourceImageData.SizeX - 1);
		Y = (int32)((uint32)Y) & (SourceImageData.SizeY - 1);
	}
	else if(AddressMode == MGTAM_Clamp)
	{
		// clamp
		X = FMath::Clamp(X, 0, SourceImageData.SizeX - 1);
		Y = FMath::Clamp(Y, 0, SourceImageData.SizeY - 1);
	}
	else if(AddressMode == MGTAM_BorderBlack)
	{
		// border color 0
		if((uint32)X >= (uint32)SourceImageData.SizeX
			|| (uint32)Y >= (uint32)SourceImageData.SizeY)
		{
			static FLinearColor Black(0, 0, 0, 0);
			return Black;
		}
	}
	else
	{
		check(0);
	}
	return SourceImageData.Access(X, Y);
}

// Same functionality as above, but for 1D lookup with explicit size
template <EMipGenAddressMode AddressMode>
static const FLinearColor& LookupSourceMip(const FLinearColor* Data, int32 Size, int32 X)
{
	if (AddressMode == MGTAM_Wrap)
	{
		// wrap
		// ! requires pow2 sizes
		checkSlow( FMath::IsPowerOfTwo(Size) );

		X = (int32)((uint32)X) & (Size - 1);
	}
	else if (AddressMode == MGTAM_Clamp)
	{
		// clamp
		X = FMath::Clamp(X, 0, Size - 1);
	}
	else if (AddressMode == MGTAM_BorderBlack)
	{
		// border color 0
		if ((uint32)X >= (uint32)Size)
		{
			static FLinearColor Black(0, 0, 0, 0);
			return Black;
		}
	}
	else
	{
		check(0);
	}
	return Data[X];
}

// Kernel class for image filtering operations like image downsampling
// at max MaxKernelExtend x MaxKernelExtend
class FImageKernel2D
{
public:
	FImageKernel2D() :FilterTableSize(0)
	{
	}

	// @param TableSize1D 2 for 2x2, 4 for 4x4, 6 for 6x6, 8 for 8x8
	// @param SharpenFactor can be negative to blur
	// generate normalized 2D Kernel with sharpening
	void BuildSeparatableGaussWithSharpen(uint32 TableSize1D, float SharpenFactor = 0.0f)
	{
		if(TableSize1D > MaxKernelExtend)
		{
			TableSize1D = MaxKernelExtend;
		}

		float* Table1D = KernelWeights1D;
		float NegativeTable1D[MaxKernelExtend];

		FilterTableSize = TableSize1D;
		
		if(TableSize1D == 2)
		{
			// 2x2 kernel: simple average
			// SharpenFactor is ignored
			// this is TMGS_SimpleAverage
			Table1D[0] = Table1D[1] = 0.5f;
			KernelWeights[0] = KernelWeights[1] = KernelWeights[2] = KernelWeights[3] = 0.25f;
			return;
		}
		else if(SharpenFactor < 0.0f)
		{
			// blur only
			// this is TMGS_Blur

			// TMGS_Blur will always give us TableSize > 2
			check( TableSize1D > 2 );

			BuildGaussian1D(Table1D, TableSize1D, 1.0f, -SharpenFactor);
			BuildFilterTable2DFrom1D(KernelWeights, Table1D, TableSize1D);
			return;
		}
		else if(TableSize1D == 4)
		{
			// 4x4 kernel with sharpen or blur: can alias a bit
			// this is not used by standard TMGS_ mip options
			//  one thing that can get you in here is GenerateTopMip
			//	because it takes the standard 8 size and does /2
			BuildFilterTable1DBase(Table1D, TableSize1D, 1.0f + SharpenFactor);
			BuildFilterTable1DBase(NegativeTable1D, TableSize1D, -SharpenFactor);
			BlurFilterTable1D(NegativeTable1D, TableSize1D, 1);
		}
		else if(TableSize1D == 6)
		{
			// 6x6 kernel with sharpen or blur: still can alias
			// this is not used by standard TMGS_ mip options
			BuildFilterTable1DBase(Table1D, TableSize1D, 1.0f + SharpenFactor);
			BuildFilterTable1DBase(NegativeTable1D, TableSize1D, -SharpenFactor);
			BlurFilterTable1D(NegativeTable1D, TableSize1D, 2);
		}
		else if(TableSize1D == 8)
		{
			//8x8 kernel with sharpen
			// these are the TMGS_Sharpen filters

			// * 2 to get similar appearance as for TableSize 6
			SharpenFactor = SharpenFactor * 2.0f;

			BuildFilterTable1DBase(Table1D, TableSize1D, 1.0f + SharpenFactor);
			// positive lobe is blurred a bit for better quality
			BlurFilterTable1D(Table1D, TableSize1D, 1);
			BuildFilterTable1DBase(NegativeTable1D, TableSize1D, -SharpenFactor);
			BlurFilterTable1D(NegativeTable1D, TableSize1D, 3);
		}
		else 
		{
			// not yet supported
			check(0);
		}

		AddFilterTable1D(Table1D, NegativeTable1D, TableSize1D);
		BuildFilterTable2DFrom1D(KernelWeights, Table1D, TableSize1D);
	}

	inline uint32 GetFilterTableSize() const
	{
		return FilterTableSize;
	}

	inline float Get1D(uint32 X) const
	{
		checkSlow(X < FilterTableSize);
		return KernelWeights1D[X];
	}

	inline float GetAt(uint32 X, uint32 Y) const
	{
		checkSlow(X < FilterTableSize);
		checkSlow(Y < FilterTableSize);
		return KernelWeights[X + Y * FilterTableSize];
	}

private:

	inline static float NormalDistribution(float X, float Variance)
	{
		const float StandardDeviation = FMath::Sqrt(Variance);
		return FMath::Exp(-FMath::Square(X) / (2.0f * Variance)) / (StandardDeviation * FMath::Sqrt(2.0f * (float)PI));
	}

	// support even and non even sized filters
	static void BuildGaussian1D(float *InOutTable, uint32 TableSize, float Sum, float Variance)
	{
		float Center = (float)TableSize * 0.5f - 0.5f;
		float CurrentSum = 0;
		for(uint32 i = 0; i < TableSize; ++i)
		{
			float Actual = NormalDistribution((float)i - Center, Variance);
			InOutTable[i] = Actual;
			CurrentSum += Actual;
		}
		// Normalize
		float InvSum = Sum / CurrentSum;
		for(uint32 i = 0; i < TableSize; ++i)
		{
			InOutTable[i] *= InvSum;
		}
	}

	//
	static void BuildFilterTable1DBase(float *InOutTable, uint32 TableSize, float Sum )
	{
		// we require a even sized filter
		check(TableSize % 2 == 0);

		float Inner = 0.5f * Sum;

		uint32 Center = TableSize / 2;
		for(uint32 x = 0; x < TableSize; ++x)
		{
			if(x == Center || x == Center - 1)
			{
				// center elements
				InOutTable[x] = Inner;
			}
			else
			{
				// outer elements
				InOutTable[x] = 0.0f;
			}
		}
	}

	// InOutTable += InTable
	static void AddFilterTable1D( float *InOutTable, const float *InTable, uint32 TableSize )
	{
		for(uint32 x = 0; x < TableSize; ++x)
		{
			InOutTable[x] += InTable[x];
		}
	}

	// @param Times 1:box, 2:triangle, 3:pow2, 4:pow3, ...
	// can be optimized with double buffering but doesn't need to be fast
	static void BlurFilterTable1D( float *InOutTable, uint32 TableSize, uint32 Times )
	{
		check(Times>0);
		check(TableSize<32);

		float Intermediate[32];

		for(uint32 Pass = 0; Pass < Times; ++Pass)
		{
			for(uint32 x = 0; x < TableSize; ++x)
			{
				Intermediate[x] = InOutTable[x];
			}

			for(uint32 x = 0; x < TableSize; ++x)
			{
				float sum = Intermediate[x];

				if(x)
				{
					sum += Intermediate[x-1];	
				}
				if(x < TableSize - 1)
				{
					sum += Intermediate[x+1];	
				}

				InOutTable[x] = sum / 3.0f;
			}
		}
	}

	static void BuildFilterTable2DFrom1D( float *OutTable2D, float *InTable1D, uint32 TableSize )
	{
		for(uint32 y = 0; y < TableSize; ++y)
		{
			for(uint32 x = 0; x < TableSize; ++x)
			{
				OutTable2D[x + y * TableSize] = InTable1D[y] * InTable1D[x];
			}
		}
	}

	// at max we support MaxKernelExtend x MaxKernelExtend kernels
	const static uint32 MaxKernelExtend = 12;
	// 0 if no kernel was setup yet
	uint32 FilterTableSize;
	// normalized, means the sum of it should be 1.0f
	float KernelWeights[MaxKernelExtend * MaxKernelExtend];
	float KernelWeights1D[MaxKernelExtend];
};

static float DetermineScaledThreshold(float Threshold, float Scale)
{
	check(Threshold > 0.f && Scale > 0.f);

	// Assuming Scale > 0 and Threshold > 0, find ScaledThreshold such that
	//	 x * Scale >= Threshold
	// is exactly equivalent to
	//	 x >= ScaledThreshold.
	//
	// This is for a test that was originally written in the first form that we want to
	// transform to the second form without changing results (which would in turn change
	// texture cooks).
	//
	// In exact arithmetic, this is just ScaledThreshold = Threshold / Scale.
	//
	// In floating point, we need to consider rounding. Computed in floating point
	// and assuming round-to-nearest (breaking ties towards even), we get 
	//
	//	 RN(x * Scale) >= Threshold
	//
	// The smallest conceivable x that passes RN(x * Scale) >= Threshold is
	// x = (Threshold - 0.5u) / Scale, landing exactly halfway with the rounding
	// going up; this is slightly less than an exact Threshold/Scale.
	//
	// For regular floating point division, we get
	//	 RN(Threshold / Scale)
	// = (Threshold / Scale) * (1 + e),  |e| < 0.5u (the inequality is strict for divisions)
	//
	// That gets us relatively close to the target value, but we have no guarantee that rounding
	// on the division was in the direction we wanted. Just check whether our target inequality
	// is satisfied and bump up or down to the next representable float as required.
	float ScaledThreshold = Threshold / Scale;
	float SteppedDown = std::nextafter(ScaledThreshold, 0.f);

	// We want ScaledThreshold to be the smallest float such that
	//	 ScaledThreshold * Scale >= Threshold
	// meaning the next-smaller float below ScaledThreshold (which is SteppedDown)
	// should not be >=Threshold. 

	if (SteppedDown * Scale >= Threshold)
	{
		// We were too large, go down by 1 ulp
		ScaledThreshold = SteppedDown;
	}
	else if (ScaledThreshold * Scale < Threshold)
	{
		// We were too small, go up by 1 ulp
		ScaledThreshold = std::nextafter(ScaledThreshold, 2.f * ScaledThreshold);
	}

	// We should now have the right threshold:
	check(ScaledThreshold * Scale >= Threshold); // ScaledThreshold is large enough
	check(std::nextafter(ScaledThreshold, 0.f) * Scale < Threshold); // next below is too small

	return ScaledThreshold;
}


static FVector4f ComputeAlphaCoverage(const FVector4f Thresholds, const FVector4f Scales, const FImageView2D& SourceImageData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.ComputeAlphaCoverage);

	FVector4f Coverage(0, 0, 0, 0);

	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob,SourceImageData.SizeX,SourceImageData.SizeY);

	if ( Thresholds[0] == 0.f && Thresholds[1] == 0.f && Thresholds[2] == 0.f )
	{
		// common case that only channel 3 (A) is used for alpha coverage :
		
		check( Thresholds[3] != 0.f );

		const float ThresholdScaled = DetermineScaledThreshold(Thresholds[3] , Scales[3]);
		
		int64 CommonResult = 0;
		ParallelFor( TEXT("Texture.ComputeAlphaCoverage.PF"),NumJobs,1, [&](int32 Index)
		{
			int32 StartIndex = Index * NumRowsEachJob;
			int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, SourceImageData.SizeY);
			int32 LocalCoverage = 0;
			for (int32 y = StartIndex; y < EndIndex; ++y)
			{
				const FLinearColor * RowPixels = &SourceImageData.Access(0,y);

				for (int32 x = 0; x < SourceImageData.SizeX; ++x)
				{
					LocalCoverage += (RowPixels[x].A >= ThresholdScaled);
				}
			}

			FPlatformAtomics::InterlockedAdd(&CommonResult, LocalCoverage);
		}, EParallelForFlags::Unbalanced);

		Coverage[3] = float(CommonResult) / float( (int64) SourceImageData.SizeX * SourceImageData.SizeY);
		
		UE_LOGF(LogTextureCompressor, VeryVerbose, "Thresholds = 000 %f Coverage = 000 %f",  \
			Thresholds[3], Coverage[3] );
	}
	else
	{
		FVector4f ThresholdsScaled;

		for (int32 i = 0; i < 4; ++i)
		{
			// Skip channel if Threshold is 0
			if (Thresholds[i] == 0)
			{
				// stuff a value that we will always be less than
				ThresholdsScaled[i] = FLT_MAX;
			}
			else
			{
				check( Scales[i] != 0.f );
				ThresholdsScaled[i] = DetermineScaledThreshold( Thresholds[i] , Scales[i] );
			}
		}

		int64 CommonResults[4] = { 0, 0, 0, 0 };
		ParallelFor( TEXT("Texture.ComputeAlphaCoverage.PF"),NumJobs,1, [&](int32 Index)
		{
			int32 StartIndex = Index * NumRowsEachJob;
			int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, SourceImageData.SizeY);
			int32 LocalCoverage[4] = { 0, 0, 0, 0 };
			for (int32 y = StartIndex; y < EndIndex; ++y)
			{
				const FLinearColor * RowPixels = &SourceImageData.Access(0,y);

				for (int32 x = 0; x < SourceImageData.SizeX; ++x)
				{
					const FLinearColor & PixelValue = RowPixels[x];

					// Calculate coverage for each channel
					for (int32 i = 0; i < 4; ++i)
					{
						LocalCoverage[i] += ( PixelValue.Component(i) >= ThresholdsScaled[i] );
					}
				}
			}

			for (int32 i = 0; i < 4; ++i)
			{
				FPlatformAtomics::InterlockedAdd(&CommonResults[i], LocalCoverage[i]);
			}
		}, EParallelForFlags::Unbalanced);

		for (int32 i = 0; i < 4; ++i)
		{
			Coverage[i] = float(CommonResults[i]) / float((int64) SourceImageData.SizeX * SourceImageData.SizeY);
		}
		
		UE_LOGF(LogTextureCompressor, VeryVerbose, "Thresholds = %f %f %f %f Coverage = %f %f %f %f",  \
			Thresholds[0], Thresholds[1], Thresholds[2], Thresholds[3], \
			Coverage[0], Coverage[1], Coverage[2], Coverage[3] );
	}
	
	return Coverage;
}

static FVector4f ComputeAlphaScale(const FVector4f Coverages, const FVector4f AlphaThresholds, const FImageView2D& SourceImageData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.ComputeAlphaScale);

	// This function is not a good way to do this
	// but we cannot change it without changing output pixels
	// A better method would be to histogram the channel and scale the histogram to meet the desired threshold
	// even if using this binary search method, you should remember which value gave the closest result
	//	 don't assume that each binary search step is an improvement
	// 

	FVector4f MinAlphaScales (0, 0, 0, 0);
	FVector4f MaxAlphaScales (4, 4, 4, 4);
	FVector4f AlphaScales (1, 1, 1, 1);

	//Binary Search to find Alpha Scale
	// limit binary search to 8 steps
	for (int32 i = 0; i < 8; ++i)
	{
		FVector4f ComputedCoverages = ComputeAlphaCoverage(AlphaThresholds, AlphaScales, SourceImageData);
		
		UE_LOGF(LogTextureCompressor, VeryVerbose, "Tried AlphaScale = %f ComputedCoverage = %f Goal = %f", AlphaScales[3], ComputedCoverages[3], Coverages[3] );

		for (int32 j = 0; j < 4; ++j)
		{
			if (AlphaThresholds[j] == 0 || fabsf(ComputedCoverages[j] - Coverages[j]) < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			if (ComputedCoverages[j] < Coverages[j])
			{
				MinAlphaScales[j] = AlphaScales[j];
			}
			else if (ComputedCoverages[j] > Coverages[j])
			{
				MaxAlphaScales[j] = AlphaScales[j];
			}

			// guess alphascale is best at next midpoint :
			//  this means we wind up returning an alphascale value we have never tested
			AlphaScales[j] = (MinAlphaScales[j] + MaxAlphaScales[j]) * 0.5f;
		}

		// Equals default tolerance is KINDA_SMALL_NUMBER so it checks the same condition as above
		if (ComputedCoverages.Equals(Coverages))
		{
			break;
		}
	}

	UE_LOGF(LogTextureCompressor, VeryVerbose, "Final AlphaScales = %f %f %f %f", AlphaScales[0], AlphaScales[1], AlphaScales[2], AlphaScales[3] );

	return AlphaScales;
}


static void GenerateMip2x2Simple(
	const FImageView2D& SourceImageData, 
	FImageView2D& DestImageData)
{
	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob,DestImageData.SizeX,DestImageData.SizeY);

	ParallelFor( TEXT("Texture.GenerateMip2x2Simple.PF"),NumJobs,1, [&](int32 Index)
	{
		int32 StartIndex = Index * NumRowsEachJob;
		int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, DestImageData.SizeY);
		for (int32 DestY = StartIndex; DestY < EndIndex; ++DestY)
		{
			float* DestRow = &DestImageData.Access(0, DestY).Component(0);
			const float* SourceRow0 = &SourceImageData.Access(0, 2*DestY).Component(0);
			const float* SourceRow1 = &SourceImageData.Access(0, 2*DestY+1).Component(0);

			const VectorRegister4Float Mul = VectorSetFloat1(0.25f);
			for (int32 DestX = 0; DestX < DestImageData.SizeX; DestX++)
			{
				VectorRegister4Float A = VectorLoad(&SourceRow0[0]);
				VectorRegister4Float B = VectorLoad(&SourceRow0[4]);
				VectorRegister4Float C = VectorLoad(&SourceRow1[0]);
				VectorRegister4Float D = VectorLoad(&SourceRow1[4]);
				VectorRegister4Float Sum = VectorAdd(VectorAdd(VectorAdd(A, B), C), D);
				VectorRegister4Float Avg = VectorMultiply(Sum, Mul);
				VectorStore(Avg, &DestRow[0]);
				SourceRow0 += 8;
				SourceRow1 += 8;
				DestRow += 4;
			}
		}
	}, EParallelForFlags::Unbalanced);
}

template <EMipGenAddressMode AddressMode>
static void GenerateMipUnfiltered(const FImageView2D& SourceImageData, FImageView2D& DestImageData, FVector4f AlphaScale, uint32 ScaleFactor)
{
	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob, DestImageData.SizeX, DestImageData.SizeY);

	ParallelFor(TEXT("Texture.GenerateMipUnfiltered.PF"), NumJobs, 1, [&](int32 Index)
	{
		VectorRegister4Float AlphaScaleV = VectorLoad(&AlphaScale[0]);
		int32 StartIndex = Index * NumRowsEachJob;
		int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, DestImageData.SizeY);
		for (int32 DestY = StartIndex; DestY < EndIndex; ++DestY)
		{
			for (int32 DestX = 0; DestX < DestImageData.SizeX; DestX++)
			{
				const int32 SourceX = DestX * ScaleFactor;
				const int32 SourceY = DestY * ScaleFactor;
				const FLinearColor& Sample = LookupSourceMip<AddressMode>(SourceImageData, SourceX, SourceY);
				VectorRegister4Float FilteredColor = VectorLoad(&Sample.Component(0));

				// Apply computed alpha scales to each channel
				FilteredColor = VectorMultiply(FilteredColor, AlphaScaleV);

				// Set the destination pixel.
				VectorStore(FilteredColor, &DestImageData.Access(DestX, DestY).Component(0));
			}
		}
	}, EParallelForFlags::Unbalanced);
}

template <EMipGenAddressMode AddressMode>
static void GenerateMip2x2(const FImageView2D& SourceImageData, FImageView2D& DestImageData, FVector4f AlphaScale, uint32 ScaleFactor)
{
	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob, DestImageData.SizeX, DestImageData.SizeY);

	ParallelFor(TEXT("Texture.GenerateMip2x2.PF"), NumJobs, 1, [&](int32 Index)
	{
		VectorRegister4Float AlphaScaleV = VectorLoad(&AlphaScale[0]);
		int32 StartIndex = Index * NumRowsEachJob;
		int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, DestImageData.SizeY);
		for (int32 DestY = StartIndex; DestY < EndIndex; ++DestY)
		{
			for (int32 DestX = 0; DestX < DestImageData.SizeX; DestX++)
			{
				const int32 SourceX = DestX * ScaleFactor;
				const int32 SourceY = DestY * ScaleFactor;

				VectorRegister4Float A = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 0, SourceY + 0).Component(0));
				VectorRegister4Float B = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 1, SourceY + 0).Component(0));
				VectorRegister4Float C = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 0, SourceY + 1).Component(0));
				VectorRegister4Float D = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 1, SourceY + 1).Component(0));
				VectorRegister4Float FilteredColor = VectorAdd(VectorAdd(VectorAdd(A, B), C), D);
				FilteredColor = VectorMultiply(FilteredColor, VectorSetFloat1(0.25f));

				// Apply computed alpha scales to each channel
				FilteredColor = VectorMultiply(FilteredColor, AlphaScaleV);

				// Set the destination pixel.
				VectorStore(FilteredColor, &DestImageData.Access(DestX, DestY).Component(0));
			}
		}
	}, EParallelForFlags::Unbalanced);
}

static void AllocateTempForMips(
	TArray64<FLinearColor>& TempData,
	int32 SourceSizeX,
	int32 SourceSizeY,
	int32 DestSizeX,
	int32 DestSizeY,
	bool bDoScaleMipsForAlphaCoverage,
	const FImageKernel2D& Kernel,
	uint32 ScaleFactor,
	bool bSharpenWithoutColorShift,
	bool bUnfiltered,
	bool bUseNewMipFilter)
{
	if (!bUseNewMipFilter)
	{
		// No need for extra memory if using old 2D filter
		return;
	}

	int32 KernelFilterTableSize = (int32)Kernel.GetFilterTableSize();

	if (KernelFilterTableSize == 2 &&
		ScaleFactor == 2 &&
		DestSizeX * 2 == SourceSizeX &&
		DestSizeY * 2 == SourceSizeY &&
		!bDoScaleMipsForAlphaCoverage &&
		!bUnfiltered)
	{
		// Will use GenerateMip2x2Simple
		return;
	}

	if (bUnfiltered)
	{
		// Will use GenerateMipUnfiltered
		return;
	}

	if (KernelFilterTableSize == 2)
	{
		// Will use GenerateMip2x2
		return;
	}

	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob, DestSizeX, DestSizeY);

	// Enough bytes to have one row per source width for each job thread
	// Make sure the rows do not overlap on same cache line
	int64 SizeInBytes = (sizeof(FLinearColor) * SourceSizeX + PLATFORM_CACHE_LINE_SIZE) * NumJobs;

	int64 ElementCount = FMath::DivideAndRoundUp<int64>(SizeInBytes, sizeof(FLinearColor));
	TempData.AddUninitialized(ElementCount);
}

template <EMipGenAddressMode AddressMode, bool bSharpenWithoutColorShift, int KernelSize = 0>
static void GenerateMipSharpened(
	const FImageView2D& SourceImageData,
	FImageView2D& DestImageData,
	const FImageKernel2D& Kernel,
	FVector4f AlphaScale,
	uint32 ScaleFactor)
{
	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob, DestImageData.SizeX, DestImageData.SizeY);

	ParallelFor(TEXT("Texture.GenerateMipSharpened.PF"), NumJobs, 1, [&](int32 Index)
	{
		// In case kernel size is passed as template argument use it as constant
		// This will allow compiler to unroll inner loops below
		const int32 KernelFilterTableSize = KernelSize ? KernelSize : (int32)Kernel.GetFilterTableSize();

		// if KernelFilterTableSize is odd, centered in-place filter can be applied
		// KernelFilterTableSize should be even for standard down-sampling
		const int32 KernelCenter = (KernelFilterTableSize - 1) / 2;

		VectorRegister4Float AlphaScaleV = VectorLoad(&AlphaScale[0]);

		int32 StartIndex = Index * NumRowsEachJob;
		int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, DestImageData.SizeY);

		for (int32 DestY = StartIndex; DestY < EndIndex; ++DestY)
		{
			for (int32 DestX = 0; DestX < DestImageData.SizeX; DestX++)
			{
				const int32 SourceX = DestX * ScaleFactor;
				const int32 SourceY = DestY * ScaleFactor;

				VectorRegister4Float Color = VectorZeroFloat();
				for (int32 KernelY = 0; KernelY < KernelFilterTableSize; ++KernelY)
				{
					for (int32 KernelX = 0; KernelX < KernelFilterTableSize; ++KernelX)
					{
						const FLinearColor& Sample = LookupSourceMip<AddressMode>(SourceImageData, SourceX + KernelX - KernelCenter, SourceY + KernelY - KernelCenter);
						VectorRegister4Float Weight = VectorSetFloat1(Kernel.GetAt(KernelX, KernelY));
						VectorRegister4Float WeightSample = VectorMultiply(Weight, VectorLoad(&Sample.Component(0)));
						Color = VectorAdd(Color, WeightSample);
					}
				}

				// This condition will be optimized away because it is constant template argument
				if (bSharpenWithoutColorShift)
				{
					VectorRegister4Float SharpenedColor = Color;

					// Luminace weights from FLinearColor::GetLuminance() function
					VectorRegister4Float LuminanceWeights = MakeVectorRegisterFloat(0.3f, 0.59f, 0.11f, 0.f);
					VectorRegister4Float NewLuminance = VectorDot3(SharpenedColor, LuminanceWeights);

					// simple 2x2 kernel to compute the color
					VectorRegister4Float A = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 0, SourceY + 0).Component(0));
					VectorRegister4Float B = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 1, SourceY + 0).Component(0));
					VectorRegister4Float C = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 0, SourceY + 1).Component(0));
					VectorRegister4Float D = VectorLoad(&LookupSourceMip<AddressMode>(SourceImageData, SourceX + 1, SourceY + 1).Component(0));
					VectorRegister4Float FilteredColor = VectorAdd(VectorAdd(VectorAdd(A, B), C), D);
					FilteredColor = VectorMultiply(FilteredColor, VectorSetFloat1(0.25f));

					VectorRegister4Float OldLuminance = VectorDot3(FilteredColor, LuminanceWeights);

					// if (OldLuminance > 0.001f) FilteredColor.RGB *= NewLuminance / OldLuminance;
					VectorRegister4Float CompareMask = VectorCompareGT(OldLuminance, VectorSetFloat1(0.001f));
					VectorRegister4Float Temp = VectorMultiply(FilteredColor, VectorDivide(NewLuminance, OldLuminance));
					FilteredColor = VectorSelect(CompareMask, Temp, FilteredColor);

					// FilteredColor.A = SharpenedColor.A
					VectorRegister4Float AlphaMask = MakeVectorRegisterFloatMask(0, 0, 0, 0xffffffff);
					FilteredColor = VectorSelect(AlphaMask, SharpenedColor, FilteredColor);

					// Apply computed alpha scales to each channel
					FilteredColor = VectorMultiply(FilteredColor, AlphaScaleV);

					// Set the destination pixel.
					VectorStore(FilteredColor, &DestImageData.Access(DestX, DestY).Component(0));
				}
				else
				{
					// Apply computed alpha scales to each channel
					Color = VectorMultiply(Color, AlphaScaleV);

					// Set the destination pixel.
					VectorStore(Color, &DestImageData.Access(DestX, DestY).Component(0));
				}
			}
		}
	}, EParallelForFlags::Unbalanced);
}

template <EMipGenAddressMode AddressMode, int KernelSize = 0>
static void GenerateMipSharpenedSeparable(
	const FImageView2D& SourceImageData,
	TArray64<FLinearColor>& TempData,
	FImageView2D& DestImageData,
	const FImageKernel2D& Kernel,
	FVector4f AlphaScale,
	uint32 ScaleFactor)
{
	int32 NumRowsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob, DestImageData.SizeX, DestImageData.SizeY);

	// Verify that caller has allocated proper amount of bytes for temporary storage
	// This is allocated in AllocateTempForMips function above
	check(TempData.Num() >= SourceImageData.SizeX * NumJobs);

	ParallelFor(TEXT("Texture.GenerateMipSharpenedSeparable.PF"), NumJobs, 1, [&](int32 Index)
	{
		// In case kernel size is passed as template argument use it as constant
		// This will allow compiler to unroll inner loops below
		const int32 KernelFilterTableSize = KernelSize ? KernelSize : (int32)Kernel.GetFilterTableSize();

		// if KernelFilterTableSize is odd, centered in-place filter can be applied
		// KernelFilterTableSize should be even for standard down-sampling
		const int32 KernelCenter = (KernelFilterTableSize - 1) / 2;

		VectorRegister4Float AlphaScaleV = VectorLoad(&AlphaScale[0]);

		int32 StartIndex = Index * NumRowsEachJob;
		int32 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, DestImageData.SizeY);

		FLinearColor* Temp = &TempData[Index * (TempData.Num() / NumJobs)];
		for (int32 DestY = StartIndex; DestY < EndIndex; ++DestY)
		{
			const int32 SourceY = DestY * ScaleFactor;

			for (int32 SourceX = 0; SourceX < SourceImageData.SizeX; SourceX++)
			{
				VectorRegister4Float Color = VectorZeroFloat();
				for (int32 KernelY = 0; KernelY < KernelFilterTableSize; ++KernelY)
				{
					const FLinearColor& Sample = LookupSourceMip<AddressMode>(SourceImageData, SourceX, SourceY + KernelY - KernelCenter);
					VectorRegister4Float Weight = VectorSetFloat1(Kernel.Get1D(KernelY));
					VectorRegister4Float WeightSample = VectorMultiply(Weight, VectorLoad(&Sample.Component(0)));
					Color = VectorAdd(Color, WeightSample);
				}
				VectorStore(Color, &Temp[SourceX].Component(0));
			}

			for (int32 DestX = 0; DestX < DestImageData.SizeX; DestX++)
			{
				const int32 SourceX = DestX * ScaleFactor;

				VectorRegister4Float Color = VectorZeroFloat();
				for (int32 KernelX = 0; KernelX < KernelFilterTableSize; ++KernelX)
				{
					const FLinearColor& Sample = LookupSourceMip<AddressMode>(Temp, SourceImageData.SizeX, SourceX + KernelX - KernelCenter);
					VectorRegister4Float Weight = VectorSetFloat1(Kernel.Get1D(KernelX));
					VectorRegister4Float WeightSample = VectorMultiply(Weight, VectorLoad(&Sample.Component(0)));
					Color = VectorAdd(Color, WeightSample);
				}

				// Apply computed alpha scales to each channel
				Color = VectorMultiply(Color, AlphaScaleV);

				// Set the destination pixel.
				VectorStore(Color, &DestImageData.Access(DestX, DestY).Component(0));
			}
		}
	}, EParallelForFlags::Unbalanced);
}

/**
* Generates a mip-map for an 2D B8G8R8A8 image using a filter with possible sharpening
* @param SourceImageData - The source image's data.
* @param TempData - Temporary storage required by separable mip filter, call AllocateTempForMips function to allocate it
* @param DestImageData - The destination image's data.
* @param ImageFormat - The format of both the source and destination images.
* @param FilterTable2D - [FilterTableSize * FilterTableSize]
* @param FilterTableSize - >= 2
* @param ScaleFactor 1 / 2:for downsampling
* @param bUseNewMipFilter - pass true to use new separable mip filter
*/
template <EMipGenAddressMode AddressMode>
static void GenerateSharpenedMipB8G8R8A8Templ(
	const FImageView2D& SourceImageData, 
	TArray64<FLinearColor>& TempData,
	FImageView2D& DestImageData, 
	bool bDoScaleMipsForAlphaCoverage,
	const FVector4f AlphaCoverages,
	const FVector4f AlphaThresholds,
	const FImageKernel2D& Kernel,
	uint32 ScaleFactor,
	bool bSharpenWithoutColorShift,
	bool bUnfiltered,
	bool bUseNewMipFilter)
{
	check( ScaleFactor == 1 || ScaleFactor == 2 );
	check( (SourceImageData.SizeX/ScaleFactor) == DestImageData.SizeX || DestImageData.SizeX == 1 );
	check( (SourceImageData.SizeY/ScaleFactor) == DestImageData.SizeY || DestImageData.SizeY == 1 );
	
	int32 KernelFilterTableSize = (int32) Kernel.GetFilterTableSize();

	checkf( KernelFilterTableSize >= 2, TEXT("Kernel table size %d, expected at least 2!"), KernelFilterTableSize);
	if ( KernelFilterTableSize == 2 )
	{
		// 2x2 is always box filter
		check( Kernel.GetAt(0,0) == 0.25f );
	}

	// Keep conditions here in sync with same conditions inside AllocateTempForMips function
	if ( KernelFilterTableSize == 2 &&
		ScaleFactor == 2 &&
		DestImageData.SizeX*2 == SourceImageData.SizeX &&
		DestImageData.SizeY*2 == SourceImageData.SizeY &&
		! bDoScaleMipsForAlphaCoverage &&
		! bUnfiltered )
	{
		// bSharpenWithoutColorShift is ignored for 2x2 filter
		GenerateMip2x2Simple(SourceImageData,DestImageData);
		return;
	}

	FVector4f AlphaScale(1, 1, 1, 1);
	if (bDoScaleMipsForAlphaCoverage)
	{
		AlphaScale = ComputeAlphaScale(AlphaCoverages, AlphaThresholds, SourceImageData);
	}

	if (bUnfiltered)
	{
		GenerateMipUnfiltered<AddressMode>(SourceImageData, DestImageData, AlphaScale, ScaleFactor);
		return;
	}

	if (KernelFilterTableSize == 2)
	{
		GenerateMip2x2<AddressMode>(SourceImageData, DestImageData, AlphaScale, ScaleFactor);
		return;
	}

	if (bUseNewMipFilter)
	{
		if (KernelFilterTableSize == 8)
		{
			GenerateMipSharpenedSeparable<AddressMode, 8>(SourceImageData, TempData, DestImageData, Kernel, AlphaScale, ScaleFactor);
		}
		else
		{
			GenerateMipSharpenedSeparable<AddressMode>(SourceImageData, TempData, DestImageData, Kernel, AlphaScale, ScaleFactor);
		}
	}
	else
	{
		if (bSharpenWithoutColorShift)
		{
			if (KernelFilterTableSize == 8)
			{
				GenerateMipSharpened<AddressMode, true, 8>(SourceImageData, DestImageData, Kernel, AlphaScale, ScaleFactor);
			}
			else
			{
				GenerateMipSharpened<AddressMode, true>(SourceImageData, DestImageData, Kernel, AlphaScale, ScaleFactor);
			}
		}
		else
		{
			if (KernelFilterTableSize == 8)
			{
				GenerateMipSharpened<AddressMode, false, 8>(SourceImageData, DestImageData, Kernel, AlphaScale, ScaleFactor);
			}
			else
			{
				GenerateMipSharpened<AddressMode, false>(SourceImageData, DestImageData, Kernel, AlphaScale, ScaleFactor);
			}
		}
	}
}

static void GenerateMipBorder(
	const FImageView2D& SrcImageData, 
	FImageView2D& DestImageData
	);

// to switch conveniently between different texture wrapping modes for the mip map generation
// the template can optimize the inner loop using a constant AddressMode
static void GenerateSharpenedMipB8G8R8A8(
	const FImageView2D& SourceImageData, 
	const FImageView2D& SourceImageData2, // Only used with volume texture.
	TArray64<FLinearColor>& TempData,
	FImageView2D& DestImageData, 
	EMipGenAddressMode AddressMode, 
	bool bDoScaleMipsForAlphaCoverage,
	FVector4f AlphaCoverages,
	FVector4f AlphaThresholds,
	const FImageKernel2D &Kernel,
	uint32 ScaleFactor,
	bool bSharpenWithoutColorShift,
	bool bUnfiltered,
	bool bUseNewMipFilter,
	bool bReDrawBorder = false
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateSharpenedMip);

	switch(AddressMode)
	{
	case MGTAM_Wrap:
		GenerateSharpenedMipB8G8R8A8Templ<MGTAM_Wrap>(SourceImageData, TempData, DestImageData, bDoScaleMipsForAlphaCoverage, AlphaCoverages, AlphaThresholds, Kernel, ScaleFactor, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
		break;
	case MGTAM_Clamp:
		GenerateSharpenedMipB8G8R8A8Templ<MGTAM_Clamp>(SourceImageData, TempData, DestImageData, bDoScaleMipsForAlphaCoverage, AlphaCoverages, AlphaThresholds, Kernel, ScaleFactor, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
		break;
	case MGTAM_BorderBlack:
		GenerateSharpenedMipB8G8R8A8Templ<MGTAM_BorderBlack>(SourceImageData, TempData, DestImageData, bDoScaleMipsForAlphaCoverage, AlphaCoverages, AlphaThresholds, Kernel, ScaleFactor, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
		break;
	default:
		check(0);
	}

	// For volume texture, do the average between the 2.
	if (SourceImageData2.IsValid() && !bUnfiltered)
	{
		FImage Temp(DestImageData.SizeX, DestImageData.SizeY, 1, ERawImageFormat::RGBA32F);
		FImageView2D TempImageData (Temp, 0);

		switch(AddressMode)
		{
		case MGTAM_Wrap:
			GenerateSharpenedMipB8G8R8A8Templ<MGTAM_Wrap>(SourceImageData2, TempData, TempImageData, bDoScaleMipsForAlphaCoverage, AlphaCoverages, AlphaThresholds, Kernel, ScaleFactor, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
			break;
		case MGTAM_Clamp:
			GenerateSharpenedMipB8G8R8A8Templ<MGTAM_Clamp>(SourceImageData2, TempData, TempImageData, bDoScaleMipsForAlphaCoverage, AlphaCoverages, AlphaThresholds, Kernel, ScaleFactor, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
			break;
		case MGTAM_BorderBlack:
			GenerateSharpenedMipB8G8R8A8Templ<MGTAM_BorderBlack>(SourceImageData2, TempData, TempImageData, bDoScaleMipsForAlphaCoverage, AlphaCoverages, AlphaThresholds, Kernel, ScaleFactor, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
			break;
		default:
			check(0);
		}

		const int32 NumColors = DestImageData.SizeX * DestImageData.SizeY;
		for (int32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
		{
			DestImageData.SliceColors[ColorIndex] =
				(DestImageData.SliceColors[ColorIndex] + TempImageData.SliceColors[ColorIndex]) * 0.5f;
		}
	}

	if ( bReDrawBorder )
	{
		GenerateMipBorder( SourceImageData, DestImageData );
	}
}

// Update border texels after normal mip map generation to preserve the colors there (useful for particles and decals).
static void GenerateMipBorder(
	const FImageView2D& SrcImageData, 
	FImageView2D& DestImageData
	)
{
	check( (SrcImageData.SizeX/2) == DestImageData.SizeX || DestImageData.SizeX == 1 );
	check( (SrcImageData.SizeY/2) == DestImageData.SizeY || DestImageData.SizeY == 1 );

	// this check is unnecessary if we always used MGTAM_Clamp here;
	bool bIsPow2 = FMath::IsPowerOfTwo(SrcImageData.SizeX) && FMath::IsPowerOfTwo(SrcImageData.SizeY);

	for ( int32 DestY = 0; DestY < DestImageData.SizeY; DestY++ )
	{
		for ( int32 DestX = 0; DestX < DestImageData.SizeX; )
		{
			FLinearColor FilteredColor(0, 0, 0, 0);
			{
				float WeightSum = 0.0f;
				for ( int32 KernelY = 0; KernelY < 2;  ++KernelY )
				{
					for ( int32 KernelX = 0; KernelX < 2;  ++KernelX )
					{
						const int32 SourceX = DestX * 2 + KernelX;
						const int32 SourceY = DestY * 2 + KernelY;

						// only average the source border
						if ( SourceX == 0 ||
							SourceX == SrcImageData.SizeX - 1 ||
							SourceY == 0 ||
							SourceY == SrcImageData.SizeY - 1 )
						{
							// I think this should have just always been MGTAM_Clamp
							//	but that changes existing content, so preserve old behavior of using _Wrap :(
							FLinearColor Sample;
							if ( bIsPow2 )
							{
								Sample = LookupSourceMip<MGTAM_Wrap>( SrcImageData, SourceX, SourceY );
							}
							else
							{
								Sample = LookupSourceMip<MGTAM_Clamp>( SrcImageData, SourceX, SourceY );
							}
							FilteredColor += Sample;
							WeightSum += 1.0f;
						}
					}
				}
				FilteredColor /= WeightSum;
			}

			// Set the destination pixel.
			//FLinearColor& DestColor = *(DestImageData.AsRGBA32F() + DestX + DestY * DestImageData.SizeX);
			FLinearColor& DestColor = DestImageData.Access(DestX, DestY);
			DestColor = FilteredColor;

			++DestX;

			if ( DestY > 0 &&
				DestY < DestImageData.SizeY - 1 &&
				DestX > 0 &&
				DestX < DestImageData.SizeX - 1 )
			{
				// jump over the non border area
				DestX += FMath::Max( 1, DestImageData.SizeX - 2 );
			}
		}
	}
}

// how should be treat lookups outside of the image
static EMipGenAddressMode ComputeAddressMode(const FImage & Image,const FTextureBuildSettings& Settings)
{
	if ( Settings.bPreserveBorder )
	{
		return Settings.bBorderColorBlack ? MGTAM_BorderBlack : MGTAM_Clamp;
	}
	
	// conditional on bUseNewMipFilter so old textures are not changed
	//	really this should be done all the time
	if ( Settings.bCubemap && Settings.bUseNewMipFilter )
	{
		// Cubemaps should Clamp on their faces, never wrap :
		return MGTAM_Clamp;
	}

	// Wrap uses AND so requires pow2 sizes ; change to Clamp if nonpow2
	bool bIsPow2 = FMath::IsPowerOfTwo(Image.SizeX) && FMath::IsPowerOfTwo(Image.SizeY);
	// 2d address mode, no need to look at volume z :
	/*
	if ( Settings.bVolume && ! FMath::IsPowerOfTwo(Image.NumSlices) )
	{
		bIsPow2 = false;
	}
	*/

	if ( ! bIsPow2 )
	{
		return MGTAM_Clamp;
	}

	// note: all textures Wrap by default even if their address mode is set to Clamp !?
	EMipGenAddressMode AddressMode = MGTAM_Wrap;

	if ( Settings.bUseNewMipFilter )
	{
		// in new filters, we do use address mode to change mipgen
		//	(isolated to new filters solely to avoid changing old content)

		// texture address mode is Wrap,Clamp,Mirror, with Wrap by default
		// treat Clamp & Mirror the same
		bool bAddressXWrap = (Settings.TextureAddressModeX == TA_Wrap);
		bool bAddressYWrap = (Settings.TextureAddressModeY == TA_Wrap);

		if ( !bAddressXWrap && !bAddressYWrap )
		{
			AddressMode = MGTAM_Clamp;
		}
		else if ( bAddressXWrap && bAddressYWrap )
		{
			AddressMode = MGTAM_Wrap;
		}
		else
		{
			// use Wrap for now to match legacy behavior
			// ideally we'd wrap one dimension and clamp the other, but that is not yet supported
			// @todo Oodle: separate wrap/clamp for X&Y ; remove the heavy templating on AddressMode
			AddressMode = MGTAM_Wrap;

			UE_LOGF(LogTextureCompressor, Verbose, "Not ideal: Heterogeneous XY Wrap/Clamp will generate mips with Wrap on both dimensions" );
		}
	}

	return AddressMode;
}

static void GenerateTopMip(const FImage& SrcImage, FImage& DestImage, const FTextureBuildSettings& Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateTopMip);

	// GenerateTopMip is only used for ApplyCompositeTexture
	// bApplyKernelToTopMip is not exposed to Texture GUI

	EMipGenAddressMode AddressMode = ComputeAddressMode(SrcImage,Settings);

	FImageKernel2D KernelDownsample;

	if ( Settings.MipSharpening < 0.f )
	{
		// negative Sharpening is a Gaussian
		//  this can make centered ("odd") filters, so the image doesn't shift
		int32 OddMipKernelSize = Settings.SharpenMipKernelSize | 1;
		KernelDownsample.BuildSeparatableGaussWithSharpen( OddMipKernelSize, Settings.MipSharpening );
	}
	else
	{	
		// non-Gaussians only support "even" filters
		//	this causes a half-pixel shift of the top mip
		// warn but then go ahead and do as requested
		UE_LOGF(LogTextureCompressor, Warning, "GenerateTopMip used with non-Gaussian blur filter will cause half pixel shift");

		KernelDownsample.BuildSeparatableGaussWithSharpen( Settings.SharpenMipKernelSize, Settings.MipSharpening );
	}

	DestImage.Init(SrcImage.SizeX, SrcImage.SizeY, SrcImage.NumSlices, SrcImage.Format, SrcImage.GammaSpace);

	const bool bSharpenWithoutColorShift = Settings.bSharpenWithoutColorShift;
	const bool bUnfiltered = Settings.MipGenSettings == TMGS_Unfiltered;
	const bool bUseNewMipFilter = Settings.bUseNewMipFilter;

	TArray64<FLinearColor> TempData;
	AllocateTempForMips(TempData, SrcImage.SizeX, SrcImage.SizeY, DestImage.SizeX, DestImage.SizeY, false, KernelDownsample, 1, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);

	for (int32 SliceIndex = 0; SliceIndex < SrcImage.NumSlices; ++SliceIndex)
	{
		FImageView2D SrcView((FImage&)SrcImage, SliceIndex);
		FImageView2D DestView(DestImage, SliceIndex);

		// generate DestImage: down sample with sharpening
		GenerateSharpenedMipB8G8R8A8(
			SrcView, 
			FImageView2D(),
			TempData,
			DestView,
			AddressMode,
			false,
			FVector4f(0, 0, 0, 0),
			FVector4f(0, 0, 0, 0),
			KernelDownsample,
			1,
			Settings.bSharpenWithoutColorShift,
			bUnfiltered,
			bUseNewMipFilter);
	}
}

static FLinearColor Bilerp(
	const FLinearColor & Sample00,
	const FLinearColor & Sample10,
	const FLinearColor & Sample01,
	const FLinearColor & Sample11,
	float FracX,float FracY)
{
	//FMath::Lerp is :
	// return (T)(A + Alpha * (B-A));
	// must match that exactly

	VectorRegister4Float V00 = VectorLoad(&Sample00.Component(0));
	VectorRegister4Float V10 = VectorLoad(&Sample10.Component(0));
	VectorRegister4Float V01 = VectorLoad(&Sample01.Component(0));
	VectorRegister4Float V11 = VectorLoad(&Sample11.Component(0));

	VectorRegister4Float S0 = VectorAdd(V00, VectorMultiply(VectorSetFloat1(FracX), VectorSubtract(V10,V00)));
	VectorRegister4Float S1 = VectorAdd(V01, VectorMultiply(VectorSetFloat1(FracX), VectorSubtract(V11,V01)));
	VectorRegister4Float SS = VectorAdd(S0 , VectorMultiply(VectorSetFloat1(FracY), VectorSubtract(S1 ,S0 )));
	FLinearColor Out;
	VectorStore(SS,&Out.Component(0));
	
	/*
	// FLinearColor has math operators but they are scalar, not vector
	FLinearColor Sample0 = FMath::Lerp(Sample00, Sample10, FracX);
	FLinearColor Sample1 = FMath::Lerp(Sample01, Sample11, FracX);
	FLinearColor Ret = FMath::Lerp(Sample0, Sample1, FracY);
	check( Out == Ret );
	*/

	return Out;
}

// pixel centers are at XY = integers
//  range of X is [-0.5,-0.5] to [Width-0.5,Height-0.5]
static FLinearColor LookupSourceMipBilinear(const FImageView2D& SourceImageData, float X, float Y)
{
	X = FMath::Clamp(X, 0.f, (float)SourceImageData.SizeX - 1.f);
	Y = FMath::Clamp(Y, 0.f, (float)SourceImageData.SizeY - 1.f);
	int32 IntX0 = FMath::TruncToInt32(X);
	int32 IntY0 = FMath::TruncToInt32(Y);
	float FractX = X - (float)IntX0;
	float FractY = Y - (float)IntY0;
	int32 IntX1 = FMath::Min(IntX0+1, SourceImageData.SizeX-1);
	int32 IntY1 = FMath::Min(IntY0+1, SourceImageData.SizeY-1);
	
	const FLinearColor & Sample00 = SourceImageData.Access(IntX0,IntY0);
	const FLinearColor & Sample10 = SourceImageData.Access(IntX1,IntY0);
	const FLinearColor & Sample01 = SourceImageData.Access(IntX0,IntY1);
	const FLinearColor & Sample11 = SourceImageData.Access(IntX1,IntY1);

	return Bilerp(Sample00,Sample10,Sample01,Sample11,FractX,FractY);
}

// UV range is [0,1] , first pixel center is at 0.5/W
static FLinearColor LookupSourceMipBilinearUV(const FImageView2D& SourceImageData, float U, float V)
{
	float X = U * (float)SourceImageData.SizeX - 0.5f;
	float Y = V * (float)SourceImageData.SizeY - 0.5f;
	return LookupSourceMipBilinear(SourceImageData,X,Y);
}

// pixel centers are at XY = integers
//  range of X is [-0.5,-0.5] to [Width-0.5,Height-0.5]
static float LookupFloatBilinear(const float * FloatPlane,int32 SizeX,int32 SizeY, float X, float Y)
{
	X = FMath::Clamp(X, 0.f, (float)SizeX - 1.f);
	Y = FMath::Clamp(Y, 0.f, (float)SizeY - 1.f);
	int32 IntX0 = FMath::TruncToInt32(X);
	int32 IntY0 = FMath::TruncToInt32(Y);
	float FractX = X - (float)IntX0;
	float FractY = Y - (float)IntY0;
	int32 IntX1 = FMath::Min(IntX0+1, SizeX-1);
	int32 IntY1 = FMath::Min(IntY0+1, SizeY-1);
	
	float Sample00 = FloatPlane[ IntX0 + IntY0 * (int64) SizeX ];
	float Sample10 = FloatPlane[ IntX1 + IntY0 * (int64) SizeX ];
	float Sample01 = FloatPlane[ IntX0 + IntY1 * (int64) SizeX ];
	float Sample11 = FloatPlane[ IntX1 + IntY1 * (int64) SizeX ];
	float Sample0 = FMath::Lerp(Sample00, Sample10, FractX);
	float Sample1 = FMath::Lerp(Sample01, Sample11, FractX);
		
	return FMath::Lerp(Sample0, Sample1, FractY);
}

// UV range is [0,1] , first pixel center is at 0.5/W
static float LookupFloatBilinearUV(const float * FloatPlane,int32 SizeX,int32 SizeY, float U, float V)
{
	float X = U * (float)SizeX - 0.5f;
	float Y = V * (float)SizeY - 0.5f;
	return LookupFloatBilinear(FloatPlane,SizeX,SizeY,X,Y);
}

struct FTextureDownscaleSettings
{
	int32 BlockSize;
	float Downscale;
	uint8 DownscaleOptions;
	bool UseNewMipFilter;

	FTextureDownscaleSettings(const FTextureBuildSettings& BuildSettings)
	{
		Downscale = BuildSettings.Downscale;
		DownscaleOptions = BuildSettings.DownscaleOptions;
		BlockSize = 4; // <- hard coded to 4 even if texture is not compressed
		UseNewMipFilter = BuildSettings.bUseNewMipFilter;
	}
};

static float GetDownscaleFinalSizeAndClampedDownscale(int32 SrcImageWidth, int32 SrcImageHeight,  const FTextureDownscaleSettings& Settings, int32& OutWidth, int32& OutHeight)
{
	check(Settings.Downscale > 1.0f); // must be already handled.
	
	float Downscale = FMath::Clamp(Settings.Downscale, 1.f, 8.f);
	
	// currently BlockSize always == 4
	// if source was blocksize aligned, make sure final size is too
	//	this is required if pixel format is DXT
	bool bKeepBlockSizeAligned = (Settings.BlockSize > 1
			&& (SrcImageWidth  % Settings.BlockSize) == 0
			&& (SrcImageHeight % Settings.BlockSize) == 0);

	int32 FinalSizeX,FinalSizeY;

	if ( Settings.UseNewMipFilter )
	{
		// new way:

		if ( bKeepBlockSizeAligned )
		{
			// choose the block count in the smaller dimension first
			// then make the larger dimension maintain aspect ratio
			//  we favor block alignment and getting the desired downscale over exactly preserving aspect ratio
			int32 FinalNumBlocksX,FinalNumBlocksY;
			if ( SrcImageWidth >= SrcImageHeight )
			{
				FinalNumBlocksY = FMath::RoundToInt( SrcImageHeight / (Downscale * Settings.BlockSize) );
				FinalNumBlocksX = FMath::RoundToInt( FinalNumBlocksY * SrcImageWidth / (float)SrcImageHeight );
			}
			else
			{
				FinalNumBlocksX = FMath::RoundToInt( SrcImageWidth / (Downscale * Settings.BlockSize) );
				FinalNumBlocksY = FMath::RoundToInt( FinalNumBlocksX * SrcImageHeight / (float)SrcImageWidth );
			}

			FinalSizeX = FMath::Max(FinalNumBlocksX,1)*Settings.BlockSize;
			FinalSizeY = FMath::Max(FinalNumBlocksY,1)*Settings.BlockSize;
		}
		else
		{
			FinalSizeX = FMath::Max(1, FMath::RoundToInt(SrcImageWidth / Downscale));
			FinalSizeY = FMath::Max(1, FMath::RoundToInt(SrcImageHeight / Downscale));
		}
	}
	else
	{
		// old way :
		//  this has the flaw of being very far away from the desired downscale when Width/Height have no good GCD

		FinalSizeX = FMath::CeilToInt((float)SrcImageWidth / Downscale);
		FinalSizeY = FMath::CeilToInt((float)SrcImageHeight / Downscale);

		if ( bKeepBlockSizeAligned )
		{
			// the following code finds non-zero dimensions of the scaled image which preserve both aspect ratio and block alignment, 
			// it favors preserving aspect ratio at the expense of not scaling the desired factor when both are not possible
			int32 GCD = FMath::GreatestCommonDivisor(SrcImageWidth, SrcImageHeight);
			int32 ScalingGridSizeX = (SrcImageWidth / GCD) * Settings.BlockSize;
			// note: more accurate would be to use (SrcImage.SizeX / Downscale) instead of FinalSizeX here
			// GridSnap rounds to nearest, and can return zero
			FinalSizeX = FMath::GridSnap(FinalSizeX, ScalingGridSizeX);
			FinalSizeX = FMath::Max(ScalingGridSizeX, FinalSizeX);
			FinalSizeY = (int32)( ((int64)FinalSizeX * SrcImageHeight) / SrcImageWidth );
			// Final Size X and Y are gauranteed to be block aligned
		}
	}

	OutWidth = FinalSizeX;
	OutHeight = FinalSizeY;
	return Downscale;
}

static void DownscaleImage(const FImage& SrcImage, FImage& DstImage, const FTextureDownscaleSettings& Settings)
{
	// BEWARE: DownscaleImage is called with SrcImage == DstImage for in-place operation
	if (Settings.Downscale <= 1.f)
	{
		return;
	}
	
	// Settings.BlockSize == 4 always, even if not compressed
	// Downscale can only be applied if NoMipMaps, otherwise it is silently ignored
	//	 also only applied if Texture2D , ignored by all other texture types

	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.DownscaleImage);

	int32 FinalSizeX = 0, FinalSizeY =0;
	float Downscale = GetDownscaleFinalSizeAndClampedDownscale(SrcImage.SizeX, SrcImage.SizeY, Settings, FinalSizeX, FinalSizeY);

	if ( Settings.UseNewMipFilter )
	{
		// changed DDC key for affected textures
				
		// choose Filter from ETextureDownscaleOptions
		FImageCore::EResizeImageFilter Filter;
		
		// unlike MinGen, the ETextureDownscaleOptions does not have "Blur" options, only Sharpen
		/*0=no sharpening but better quality which is softer, 1=little, 5=medium, 10=extreme. */
		
		switch( (ETextureDownscaleOptions)Settings.DownscaleOptions )
		{
		case ETextureDownscaleOptions::Default:
			Filter = FImageCore::EResizeImageFilter::CubicMitchell;
			break;
		case ETextureDownscaleOptions::Unfiltered:
			Filter = FImageCore::EResizeImageFilter::PointSample;
			break;
		case ETextureDownscaleOptions::SimpleAverage:
			Filter = FImageCore::EResizeImageFilter::Triangle;
			break;
			
		// cubic Mitchells in order of B: (from highest B to lowest; eg. smoothest to sharpest)
		// CubicGaussian  // B = 1
		// CubicMitchell // B = 1/3
		// MitchellOneQuarter // B = 1/4
		// MitchellOneSixth // B = 1/6
		// CubicSharp  // B = 0

		case ETextureDownscaleOptions::Sharpen0:
			Filter = FImageCore::EResizeImageFilter::CubicGaussian;
			break;
		case ETextureDownscaleOptions::Sharpen1:
		case ETextureDownscaleOptions::Sharpen2:
			Filter = FImageCore::EResizeImageFilter::CubicMitchell;
			break;
		case ETextureDownscaleOptions::Sharpen3:
		case ETextureDownscaleOptions::Sharpen4:
			Filter = FImageCore::EResizeImageFilter::MitchellOneQuarter;
			break;
		case ETextureDownscaleOptions::Sharpen5:
		case ETextureDownscaleOptions::Sharpen6:
			Filter = FImageCore::EResizeImageFilter::MitchellOneSixth;
			break;			
		case ETextureDownscaleOptions::Sharpen7:
		case ETextureDownscaleOptions::Sharpen8:
			Filter = FImageCore::EResizeImageFilter::CubicSharp;
			break;
		case ETextureDownscaleOptions::Sharpen9:
			//Filter = FImageCore::EResizeImageFilter::Lanczos4;
			Filter = FImageCore::EResizeImageFilter::MitchellNegOneSixth;
			break;
		case ETextureDownscaleOptions::Sharpen10:
			//Filter = FImageCore::EResizeImageFilter::Lanczos5;
			Filter = FImageCore::EResizeImageFilter::MitchellNegOneThird;
			break;

		default:
			UE_LOGF(LogTextureCompressor, Warning, "Downscale option not recognized : %d",(int)Settings.DownscaleOptions);
			Filter = FImageCore::EResizeImageFilter::CubicMitchell;
			break;
		}

		FImage Temp; // needed because Src == Dst
		FImageCore::ResizeImageAllocDest(SrcImage,Temp,FinalSizeX,FinalSizeY,Filter);
		Temp.Swap(DstImage);
		return;
	}
	
	// legacy/bad path , not used for new textures that have UseNewMipFilter set
	//	left as-is to prevent changing old content

	// what this function does is 2X downsamples with the mip filter
	//	and then a final bilinear resize to the desired final size
	// instead just use ResizeImage to go directly from source size to final size in one step
	
	// recompute Downscale factor because it may have changed due to block alignment
	// note: if aspect ratio was not exactly preserved, this could differ in X and Y
	Downscale = (float)SrcImage.SizeX / (float)FinalSizeX;
	
	// while desired final size is > 2X smaller, do 2X resizes using the standard mip gen filter code

	FImage Image0;
	FImage Image1;
	FImage* ImageChain[2] = {&const_cast<FImage&>(SrcImage), &Image1};
	const bool bUnfiltered = Settings.DownscaleOptions == (uint8)ETextureDownscaleOptions::Unfiltered;
	const bool bUseNewMipFilter = Settings.UseNewMipFilter;
	
	// Scaledown using 2x2 average, use user specified filtering only for last iteration
	FImageKernel2D AvgKernel;
	AvgKernel.BuildSeparatableGaussWithSharpen(2);

	TArray64<FLinearColor> TempData;
	AllocateTempForMips(TempData, SrcImage.SizeX, SrcImage.SizeY, FMath::Max(1, SrcImage.SizeX / 2), FMath::Max(1, SrcImage.SizeY / 2), false, AvgKernel, 2, false, bUnfiltered, bUseNewMipFilter);

	int32 NumIterations = 0;
	while(Downscale > 2.0f) // NOT == 2.0
	{
		int32 DstSizeX = FMath::Max(1, ImageChain[0]->SizeX / 2 );
		int32 DstSizeY = FMath::Max(1, ImageChain[0]->SizeY / 2 );
		ImageChain[1]->Init(DstSizeX, DstSizeY, ImageChain[0]->NumSlices, ImageChain[0]->Format, ImageChain[0]->GammaSpace);

		FImageView2D SrcImageData(*ImageChain[0], 0);
		FImageView2D DstImageData(*ImageChain[1], 0);
		GenerateSharpenedMipB8G8R8A8Templ<MGTAM_Clamp>(
			SrcImageData, 
			TempData,
			DstImageData, 
			false,
			FVector4f(0, 0, 0, 0),
			FVector4f(0, 0, 0, 0), 
			AvgKernel, 
			2, 
			false,
			bUnfiltered,
			bUseNewMipFilter);

		if (NumIterations == 0)
		{
			ImageChain[0] = &Image0;
		}
		Swap(ImageChain[0], ImageChain[1]);
		
		NumIterations++;
		Downscale/= 2.f;
	}

	if (ImageChain[0]->SizeX == FinalSizeX &&
		ImageChain[0]->SizeY == FinalSizeY)
	{
		ImageChain[0]->CopyTo(DstImage, ImageChain[0]->Format, ImageChain[0]->GammaSpace);
		return;
	}
	
	int32 KernelSize = 2;
	float Sharpening = 0.0f;
	bool bBilinear = false;
	if (Settings.DownscaleOptions >= (uint8)ETextureDownscaleOptions::Sharpen0 && Settings.DownscaleOptions <= (uint8)ETextureDownscaleOptions::Sharpen10)
	{
		// 0 .. 2.0f
		Sharpening = (float)((int32)Settings.DownscaleOptions - (int32)ETextureDownscaleOptions::Sharpen0) * 0.2f;
		KernelSize = 8;
	}
	else
	{
		// Default should have mapped to SimpleAverage before getting here (in GetDownscaleOptions)
		check( Settings.DownscaleOptions != (uint8)ETextureDownscaleOptions::Default );

		bBilinear = Settings.DownscaleOptions == (uint8)ETextureDownscaleOptions::SimpleAverage;
		check( bBilinear || bUnfiltered );
	}	
	
	FImageKernel2D KernelSharpen;
	KernelSharpen.BuildSeparatableGaussWithSharpen(KernelSize, Sharpening);
	const int32 KernelCenter = (int32)KernelSharpen.GetFilterTableSize() / 2 - 1;
		
	ImageChain[1] = &DstImage;
	if (ImageChain[0] == ImageChain[1])
	{
		ImageChain[0]->CopyTo(Image0, ImageChain[0]->Format, ImageChain[0]->GammaSpace);
		ImageChain[0] = &Image0;
	}
	
	ImageChain[1]->Init(FinalSizeX, FinalSizeY, ImageChain[0]->NumSlices, ImageChain[0]->Format, ImageChain[0]->GammaSpace);

	// recompute Downscale factor again for final arbitrary-factor resizes :
	// note: if aspect ratio was not exactly preserved, this could differ in X and Y
	Downscale = (float)ImageChain[0]->SizeX / (float)FinalSizeX;

	FImageView2D SrcImageData(*ImageChain[0], 0);
	FImageView2D DstImageData(*ImageChain[1], 0);
				
	// this is not a correct image resize without shift; does not get pixel center offsets right
	//	this is in the legacy/deprecated path so leave as-is ; new path is correct
	for (int32 Y = 0; Y < FinalSizeY; ++Y)
	{
		float SourceY = (float)Y * Downscale;
		int32 IntSourceY = FMath::RoundToInt(SourceY);
		
		for (int32 X = 0; X < FinalSizeX; ++X)
		{
			float SourceX = (float)X * Downscale;

			FLinearColor FilteredColor(0,0,0,0);

			if (bUnfiltered)
			{
				int32 IntSourceX = FMath::RoundToInt(SourceX);
				FilteredColor = LookupSourceMip<MGTAM_Clamp>(SrcImageData, IntSourceX, IntSourceY);
			}
			else if(bBilinear)
			{
				// in the common case of Downscale == 2.0
				//	this is actually not bilinar at all, or a 2x2 downsample
				//	it's just point sampled
				//	because of the way the pixel centers are not offset correctly
				//	this is just sampling pixels at 0,2,4,..
				//FilteredColor = LookupSourceMipBilinear(SrcImageData, SourceX + 0.5, SourceY + 0.5); // <- correct offsets for Downscale == 2.0
				FilteredColor = LookupSourceMipBilinear(SrcImageData, SourceX, SourceY);
			}
			else
			{
				for (uint32 KernelY = 0; KernelY < KernelSharpen.GetFilterTableSize();  ++KernelY)
				{
					for (uint32 KernelX = 0; KernelX < KernelSharpen.GetFilterTableSize();  ++KernelX)
					{
						float Weight = KernelSharpen.GetAt(KernelX, KernelY);
						FLinearColor Sample = LookupSourceMipBilinear(SrcImageData, SourceX + (float)KernelX - (float)KernelCenter, SourceY + (float)KernelY - (float)KernelCenter);
						FilteredColor += Weight	* Sample;
					}
				}
			}

			// Set the destination pixel.
			FLinearColor& DestColor = DstImageData.Access(X, Y);
			DestColor = FilteredColor;
		}
	}
}

// Replaces previous calls to FImage::Linearize and FImageCore::TransformToWorkingColorSpace
static void LinearizeToWorkingColorSpace(const FImage& SrcImage, FImage& DstImage, const FTextureBuildSettings& BuildSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.LinearizeToWorkingColorSpace);

	// Allocate the 32-bit linear floating-point image
	DstImage.Init(SrcImage.SizeX, SrcImage.SizeY, SrcImage.NumSlices, ERawImageFormat::RGBA32F, EGammaSpace::Linear);

	FImageView SrcImageView(SrcImage);
	const UE::Color::EEncoding SourceEncodingOverride = static_cast<UE::Color::EEncoding>(BuildSettings.SourceEncodingOverride);
	
	if ( ! BuildSettings.bHasColorSpaceDefinition )
	{
		if ( SourceEncodingOverride == UE::Color::EEncoding::Linear )
		{
			SrcImageView.GammaSpace = EGammaSpace::Linear;
			FImageCore::CopyImage(SrcImageView, DstImage);
			return;
		}
		else if ( SourceEncodingOverride == UE::Color::EEncoding::sRGB 
			&& ERawImageFormat::GetFormatNeedsGammaSpace(SrcImage.Format) )
		{
			SrcImageView.GammaSpace = EGammaSpace::sRGB;
			FImageCore::CopyImage(SrcImageView, DstImage);
			return;
		}
		else if ( SourceEncodingOverride == UE::Color::EEncoding::Gamma22 
			&& ERawImageFormat::GetFormatNeedsGammaSpace(SrcImage.Format) )
		{
			SrcImageView.GammaSpace = EGammaSpace::Pow22;
			FImageCore::CopyImage(SrcImageView, DstImage);
			return;
		}
		// could also early out for Encoding == None, but that's handled below
	}

	// If the source encoding override is active, we avoid CopyImage's de-gammatization and instead let OpenColorIO do the decoding below.
	if (SourceEncodingOverride != UE::Color::EEncoding::None)
	{
		SrcImageView.GammaSpace = DstImage.GammaSpace; // EGammaSpace::Linear
	}

	// CopyImage to get pixels in RGAB32F , then OCIO will act on those in-place in DstImage
	FImageCore::CopyImage(SrcImageView, DstImage);

	// Decode and/or color transform to the working color space when needed
	if (SourceEncodingOverride != UE::Color::EEncoding::None || BuildSettings.bHasColorSpaceDefinition)
	{
		FOpenColorIOWrapperSourceColorSettings SrcSettings;
		SrcSettings.EncodingOverride = SourceEncodingOverride;

		if (BuildSettings.bHasColorSpaceDefinition)
		{
			// Matrix transform will be applied by OpenColorIO, unless it is already equal to the working color space.
			// Note: We no longer manually apply SaturateToHalfFloat(..) as this is now handled internally by the library.
			TStaticArray<FVector2d, 4> SourceChromaticities;
			SourceChromaticities[0] = FVector2d(BuildSettings.RedChromaticityCoordinate);
			SourceChromaticities[1] = FVector2d(BuildSettings.GreenChromaticityCoordinate);
			SourceChromaticities[2] = FVector2d(BuildSettings.BlueChromaticityCoordinate);
			SourceChromaticities[3] = FVector2d(BuildSettings.WhiteChromaticityCoordinate);

			SrcSettings.ColorSpaceOverride = MoveTemp(SourceChromaticities);
			SrcSettings.ChromaticAdaptationMethod = static_cast<UE::Color::EChromaticAdaptationMethod>(BuildSettings.ChromaticAdaptationMethod);
		}

		// Invoke the OpenColorIO library processor with the specified transformation to the working color space.
		const FOpenColorIOWrapperProcessor Processor = FOpenColorIOWrapperProcessor::CreateTransformToWorkingColorSpace(SrcSettings);
		const bool bSuccess = Processor.TransformImage(DstImage);
		if (!bSuccess)
		{
			UE_LOGF(LogTextureCompressor, Error, "Failed to linearize the texture to the working color space.");
		}
	}
}

void ITextureCompressorModule::GenerateMipChain(
	const FTextureBuildSettings& Settings,
	const FImage& BaseImage,
	TArray<FImage> &OutMipChain,
	uint32 MipChainDepth 
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateMipChain);
	
	// maybe someday : add a true 3D kernel for Volume Textures?

	// MipChainDepth is the number more to make, OutMipChain has some already
	// typically BaseImage == OutMipChain.Last()
	if ( MipChainDepth == 0 )
	{
		return;
	}

	check(BaseImage.Format == ERawImageFormat::RGBA32F && BaseImage.GammaSpace == EGammaSpace::Linear);

	const FImage& BaseMip = BaseImage;
	const int32 SrcWidth = BaseMip.SizeX;
	const int32 SrcHeight= BaseMip.SizeY;
	const int32 SrcNumSlices = BaseMip.NumSlices;
	const ERawImageFormat::Type ImageFormat = ERawImageFormat::RGBA32F;

	// Filtering kernels.
	FImageKernel2D KernelSimpleAverage;
	FImageKernel2D KernelDownsample;
	KernelSimpleAverage.BuildSeparatableGaussWithSharpen( 2 );
	KernelDownsample.BuildSeparatableGaussWithSharpen( Settings.SharpenMipKernelSize, Settings.MipSharpening );

	bool bDownsampleWithAverage = Settings.bDownsampleWithAverage;
	if ( Settings.SharpenMipKernelSize == 2 )
	{
		// filter is simple 2x2
		//	no need to do the primary filter and also a 2x2 downfilter
		//	if the primary filter IS 2x2
		bDownsampleWithAverage = false;
	}
	
	/**
	 
	 GenerateMipChain

	 has two distinct modes of operation

	 if !bDownsampleWithAverage
	 then the mip filter is used to both make the output mips, and to make the parents down the chain
	 in this case no temp images are needed
	 there is only one mipgen per mip level
	 each mipgen is dependent on the previous
	 this is used for 2x2 downsample and blurs

	 if bDownsampleWithAverage
	 then the chosen mip filter is used to make output mips from its parent
	 but a different filter (2x2) is used to make the parents down the chain
	 two mipgens are done per level
	 two temp images are used as a pingpong swap of parent/child down the chain
	 this is used for the "sharpen" family of filters

	 every mip level is made twice :

		[0] -> [1] -> [2] -> [3] -> .. with SimpleAverage 2x2 ; these are stored in FirstTemp/SecondTemp
		  \      \      \
		   \      \      \
			[1]    [2]    [3]   .. with Sharpen ; these go in OutMipChain


	 */

	const FImage* IntermediateSrcPtr = nullptr;
	FImage* IntermediateDstPtr = nullptr;

	// This will be used as a buffer for the mip processing
	FImage FirstTempImage;

	// Source for first mip step is the passed-in image:
	IntermediateSrcPtr = &BaseMip;

	// The image for the first destination
	FImage SecondTempImage;
	
	if ( bDownsampleWithAverage )
	{
		SecondTempImage.Init(FMath::Max<uint32>( 1, SrcWidth >> 1 ), FMath::Max<uint32>( 1, SrcHeight >> 1 ), Settings.bVolume ? FMath::Max<uint32>( 1, SrcNumSlices >> 1 ) : SrcNumSlices, ImageFormat);

		// This temp image will be first used as an intermediate destination for the third mip in the chain
		FirstTempImage.Init( FMath::Max<uint32>( 1, SrcWidth >> 2 ), FMath::Max<uint32>( 1, SrcHeight >> 2 ), Settings.bVolume ? FMath::Max<uint32>( 1, SrcNumSlices >> 2 ) : SrcNumSlices, ImageFormat );

		IntermediateDstPtr = &SecondTempImage;
	}

	EMipGenAddressMode AddressMode = ComputeAddressMode(*IntermediateSrcPtr,Settings);
	bool bReDrawBorder = false;
	if( Settings.bPreserveBorder )
	{
		bReDrawBorder = !Settings.bBorderColorBlack;
	}

	// Calculate alpha coverage value to preserve along mip chain
	FVector4f AlphaCoverages(0, 0, 0, 0);
	if ( Settings.bDoScaleMipsForAlphaCoverage )
	{
		check(Settings.AlphaCoverageThresholds != FVector4f(0,0,0,0));
		check(IntermediateSrcPtr);
		const FImageView2D IntermediateSrcView = FImageView2D::ConstructConst(*IntermediateSrcPtr, 0);

		const FVector4f AlphaScales(1, 1, 1, 1);		
		AlphaCoverages = ComputeAlphaCoverage(Settings.AlphaCoverageThresholds, AlphaScales, IntermediateSrcView);
	}
	
	int32 FirstMipSizeX = FMath::Max(BaseMip.SizeX>>1,1);
	int32 FirstMipSizeY = FMath::Max(BaseMip.SizeY>>1,1);

	TArray64<FLinearColor> DownsampleTempData;
	TArray64<FLinearColor> AverageTempData;
	const bool bDoScaleMipsForAlphaCoverage = Settings.bDoScaleMipsForAlphaCoverage;
	const bool bSharpenWithoutColorShift = Settings.bSharpenWithoutColorShift;
	const bool bUnfiltered = Settings.MipGenSettings == TMGS_Unfiltered;
	const bool bUseNewMipFilter = Settings.bUseNewMipFilter;
	AllocateTempForMips(DownsampleTempData, BaseMip.SizeX, BaseMip.SizeY, FirstMipSizeX, FirstMipSizeY, bDoScaleMipsForAlphaCoverage, KernelDownsample, 2, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);

	if ( bDownsampleWithAverage )
	{
		AllocateTempForMips(AverageTempData, BaseMip.SizeX, BaseMip.SizeY, FirstMipSizeX, FirstMipSizeY, bDoScaleMipsForAlphaCoverage, KernelSimpleAverage, 2, bSharpenWithoutColorShift, bUnfiltered, bUseNewMipFilter);
	}

	// Generate mips
	//  default value of MipChainDepth is MAX_uint32, means generate all mips down to 1x1
	//	(break inside the loop)
	for (; MipChainDepth != 0 ; --MipChainDepth)
	{
		check(IntermediateSrcPtr);
		const FImage& IntermediateSrc = *IntermediateSrcPtr;
		
		if ( IntermediateSrc.SizeX == 1 && IntermediateSrc.SizeY == 1 && (!Settings.bVolume || IntermediateSrc.NumSlices == 1))
		{
			// should not have been called, starting mip is already small enough
			check(0);
			break;
		}
		
		int32 DstSizeX = FMath::Max( 1, IntermediateSrc.SizeX >> 1 );
		int32 DstSizeY = FMath::Max( 1, IntermediateSrc.SizeY >> 1 );
		int32 DstNumSlices = Settings.bVolume ? FMath::Max( 1, IntermediateSrc.NumSlices >> 1 ) : SrcNumSlices;

		if ( bDownsampleWithAverage )
		{
			// IntermediateSrc & Dest = First/Second Temp Image, swapping at each step
			// IntermediateSrc/Dest are used to generate down the chain

			check(IntermediateDstPtr);

			// Update the destination size for the next iteration.
			IntermediateDstPtr->SizeX = DstSizeX;
			IntermediateDstPtr->SizeY = DstSizeY;
			IntermediateDstPtr->NumSlices = DstNumSlices;
		}

		// add new mip to TArray<FImage> &OutMipChain :
		//	placement new on TArray does AddUninitialized then constructs in the last element
		check( OutMipChain.GetSlack() > 0 ); // should have been reserved
		FImage& DestImage = OutMipChain.Emplace_GetRef(DstSizeX, DstSizeY, DstNumSlices, ImageFormat);
		
		for (int32 SliceIndex = 0; SliceIndex < DstNumSlices; ++SliceIndex)
		{
			const int32 SrcSliceIndex = Settings.bVolume ? (SliceIndex * 2) : SliceIndex;
			const FImageView2D IntermediateSrcView = FImageView2D::ConstructConst(IntermediateSrc, SrcSliceIndex);
			FImageView2D IntermediateSrcView2;
			if ( Settings.bVolume )
			{
				const int32 SrcSliceIndex2 = FMath::Min(SrcSliceIndex + 1,IntermediateSrc.NumSlices-1);
				IntermediateSrcView2 = FImageView2D::ConstructConst(IntermediateSrc, SrcSliceIndex2);
			}
			FImageView2D DestView(DestImage, SliceIndex);

			// lambda to generate one step :
			auto MakeDestImageMip = [&](FImageView2D & ArgDestView,TArray64<FLinearColor> & ArgTempData,FImageKernel2D & ArgKernel) {
				// DestView is the output mip
				GenerateSharpenedMipB8G8R8A8(
					IntermediateSrcView, 
					IntermediateSrcView2,
					ArgTempData,
					ArgDestView,
					AddressMode,
					bDoScaleMipsForAlphaCoverage,
					AlphaCoverages,
					Settings.AlphaCoverageThresholds,
					ArgKernel,
					2,
					bSharpenWithoutColorShift,
					bUnfiltered,
					bUseNewMipFilter,
					bReDrawBorder);			
			};

			// generate IntermediateDstImage:
			// IntermediateDstImage will be the source for the next mip
			if ( bDownsampleWithAverage )
			{
				// make an async task to generate the output mip:
				UE::Tasks::TTask<void> MakeDestImageMipTask = UE::Tasks::Launch( TEXT("MakeDestImageMip.Task"), [&](){
					MakeDestImageMip(DestView,DownsampleTempData,KernelDownsample);
				} );
			
				FImageView2D IntermediateDstView(*IntermediateDstPtr, SliceIndex);

				// down sample with 2x2 for the next iteration
				//	output of this is used as source for the next level
				MakeDestImageMip(IntermediateDstView,AverageTempData,KernelSimpleAverage);

				// wait on the task :
				// we don't depend on the output of this task, so waiting on that is not needed
				//   but we do use the source image for the pingpong buffer, so you have to wait before reusing that memory
				// in theory there are ways we could remove this wait (delay it until the whole function returns)
				//   (one important case is the very first level where Source == BaseImage is not the pingpong)
				// but then you also have to address the lambda using local variables/lifetimes/etc.
				MakeDestImageMipTask.GetResult();
			}
			else
			{
				// single synchronous mipgen per level
				// generate mips directly in the output chain

				MakeDestImageMip(DestView,DownsampleTempData,KernelDownsample);
			}
		}

		// Once we've created mip-maps down to 1x1, we're done.
		if ( DstSizeX == 1 && DstSizeY == 1 && (!Settings.bVolume || DstNumSlices == 1))
		{
			break;
		}
		
		if ( bDownsampleWithAverage )
		{
			// last destination becomes next source
			if (IntermediateDstPtr == &SecondTempImage)
			{
				IntermediateDstPtr = &FirstTempImage;
				IntermediateSrcPtr = &SecondTempImage;
			}
			else
			{
				IntermediateDstPtr = &SecondTempImage;
				IntermediateSrcPtr = &FirstTempImage;
			}
		}
		else
		{
			// point at the mip we made in OutMipChain
			//  safe because OutMipChain is ensured to not reallocate
			IntermediateSrcPtr = &DestImage;
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateMipChain.Free);
		
		// this is significant; detach trick again?

		/*
		FirstTempImage.RawData.Empty();
		SecondTempImage.RawData.Empty();		
		DownsampleTempData.Empty();
		AverageTempData.Empty();
		*/

		struct JunkToFree
		{
			TArray64<uint8> A;
			TArray64<uint8> B;
			TArray<FLinearColor> C;
			TArray<FLinearColor> D;
		};
		
		JunkToFree * Junk = new JunkToFree;
		Junk->A = MoveTemp( FirstTempImage.RawData );
		Junk->B = MoveTemp( SecondTempImage.RawData );
		Junk->C = MoveTemp( DownsampleTempData );
		Junk->D = MoveTemp( AverageTempData );
			
		UE::Tasks::Launch(TEXT("Texture.GenerateMipChain.Free"),
			[Junk]() // by value, not ref
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateMipChain.Free);
				delete Junk;
			},
			LowLevelTasks::ETaskPriority::BackgroundNormal);
	}
}

/*------------------------------------------------------------------------------
	Angular Filtering for HDR Cubemaps.
------------------------------------------------------------------------------*/

/**
 * View in to an image that allows access by converting a direction to longitude and latitude.
 */
struct FImageViewLongLat
{
	/** Image colors. */
	FLinearColor* ImageColors;
	/** Width of the image. */
	int64 SizeX;
	/** Height of the image. */
	int64 SizeY;

	/** Initialization constructor. */
	explicit FImageViewLongLat(FImage& Image, int32 SliceIndex)
	{
		SizeX = Image.SizeX;
		SizeY = Image.SizeY;
		ImageColors = (&Image.AsRGBA32F()[0]) + SliceIndex * SizeY * SizeX;
	}

	/** Const access to a texel. */
	const FLinearColor & Access(int32 X, int32 Y) const
	{
		return ImageColors[X + Y * SizeX];
	}

	/** Makes a filtered lookup. */
	FLinearColor LookupFiltered(float X, float Y) const
	{
		// X is in (0,SizeX) (exclusive)
		// Y is in (0,SizeY)
		checkSlow( X > 0.f && X < SizeX );
		checkSlow( Y > 0.f && Y < SizeY );

		int32 X0 = (int32)X;
		int32 Y0 = (int32)Y;

		float FracX = X - (float)X0;
		float FracY = Y - (float)Y0;

		int32 X1 = X0 + 1;
		int32 Y1 = Y0 + 1;

		// wrap X :
		checkSlow( X0 >= 0 && X0 < SizeX );
		if ( X1 >= SizeX )
		{
			X1 = 0;
		}

		// clamp Y :
		// clamp should only ever change SizeY to SizeY -1 ?
		checkSlow( Y0 >= 0 && Y0 < SizeY );
		if ( Y1 >= SizeY )
		{
			Y1 = SizeY-1;
		}

		const FLinearColor & CornerRGB00 = Access(X0, Y0);
		const FLinearColor & CornerRGB10 = Access(X1, Y0);
		const FLinearColor & CornerRGB01 = Access(X0, Y1);
		const FLinearColor & CornerRGB11 = Access(X1, Y1);

		return Bilerp(CornerRGB00,CornerRGB10,CornerRGB01,CornerRGB11,FracX,FracY);
	}

	/** Makes a filtered lookup using a direction. */
	// see http://gl.ict.usc.edu/Data/HighResProbes
	// latitude-longitude panoramic format = equirectangular mapping

	// using floats saves ~6% of the total time
	//	but would change output
	// cycles_float = 135 cycles_double = 143
	FLinearColor LookupLongLatFloat(const FVector & NormalizedDirection) const
	{
		// atan2 returns in [-PI,PI]
		// acos returns in [0,PI]
		const FVector3f NormalizedDirectionFloat { NormalizedDirection };

		const float invPI = 1.f/PI;
		float X = (1.f + atan2f(NormalizedDirectionFloat.X, -NormalizedDirectionFloat.Z) * invPI) * 0.5f * (float)SizeX;
		float Y = acosf(NormalizedDirectionFloat.Y)*invPI * (float)SizeY;

		return LookupFiltered(X, Y);
	}
	
	FLinearColor LookupLongLatDouble(const FVector & NormalizedDirection) const
	{
		// this does the math in doubles then stores to floats :
		//	that was probably a mistake, but leave it to avoid patches
		float X = (float)((1 + atan2(NormalizedDirection.X, -NormalizedDirection.Z) / PI) / 2 * (float)SizeX);
		float Y = (float)(acos(NormalizedDirection.Y) / PI * (float)SizeY);

		return LookupFiltered(X, Y);
	}
};

// transform world space vector to a space relative to the face
static inline FVector TransformSideToWorldSpace(uint32 CubemapFace, const FVector & InDirection)
{
	double x = InDirection.X, y = InDirection.Y, z = InDirection.Z;

	FVector Ret;

	// see http://msdn.microsoft.com/en-us/library/bb204881(v=vs.85).aspx
	switch(CubemapFace)
	{
		default: checkSlow(0);
		case 0: Ret = FVector(+z, -y, -x); break;
		case 1: Ret = FVector(-z, -y, +x); break;
		case 2: Ret = FVector(+x, +z, +y); break;
		case 3: Ret = FVector(+x, -z, -y); break;
		case 4: Ret = FVector(+x, -y, +z); break;
		case 5: Ret = FVector(-x, -y, -z); break;
	}

	// this makes it with the Unreal way (z and y are flipped)
	return FVector(Ret.X, Ret.Z, Ret.Y);
}

// transform vector relative to the face to world space
static inline FVector TransformWorldToSideSpace(uint32 CubemapFace, const FVector & InDirection)
{
	// undo Unreal way (z and y are flipped)
	double x = InDirection.X, y = InDirection.Z, z = InDirection.Y;

	FVector Ret;

	// see http://msdn.microsoft.com/en-us/library/bb204881(v=vs.85).aspx
	switch(CubemapFace)
	{
		default: checkSlow(0);
		case 0: Ret = FVector(-z, -y, +x); break;
		case 1: Ret = FVector(+z, -y, -x); break;
		case 2: Ret = FVector(+x, +z, +y); break;
		case 3: Ret = FVector(+x, -z, -y); break;
		case 4: Ret = FVector(+x, -y, +z); break;
		case 5: Ret = FVector(-x, -y, -z); break;
	}

	return Ret;
}

static inline FVector ComputeSSCubeDirectionAtTexelCenter(uint32 x, uint32 y, float InvSideExtent)
{
	// center of the texels
	FVector DirectionSS(((float)x + 0.5f) * InvSideExtent * 2 - 1, ((float)y + 0.5f) * InvSideExtent * 2 - 1, 1);
	DirectionSS.Normalize();
	return DirectionSS;
}

static inline FVector ComputeWSCubeDirectionAtTexelCenter(uint32 CubemapFace, uint32 x, uint32 y, float InvSideExtent)
{
	FVector DirectionSS = ComputeSSCubeDirectionAtTexelCenter(x, y, InvSideExtent);
	FVector DirectionWS = TransformSideToWorldSpace(CubemapFace, DirectionSS);
	return DirectionWS;
}

void ITextureCompressorModule::GenerateBaseCubeMipFromLongitudeLatitude2D(FImage* OutMip, const FImage& SrcImage, const uint32 MaxCubemapTextureResolution, uint8 SourceEncodingOverride)
{
	FTextureBuildSettings TempBuildSettings;
	TempBuildSettings.MaxTextureResolution = MaxCubemapTextureResolution;
	TempBuildSettings.SourceEncodingOverride = SourceEncodingOverride;
	TempBuildSettings.bHasColorSpaceDefinition = false;

	GenerateBaseCubeMipFromLongitudeLatitude2D(OutMip, SrcImage, TempBuildSettings);
}

void ITextureCompressorModule::GenerateBaseCubeMipFromLongitudeLatitude2D(FImage* OutMip, const FImage& SrcImage, const FTextureBuildSettings& InBuildSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CubeMipFromLongLat);

	FImage LongLatImage;
	LinearizeToWorkingColorSpace(SrcImage, LongLatImage, InBuildSettings);

	// TODO_TEXTURE: Expose target size to user.
	uint32 Extent = UE::TextureBuildUtilities::ComputeLongLatCubemapExtents(LongLatImage.SizeX, InBuildSettings.MaxTextureResolution);
	float InvExtent = 1.0f / (float)Extent;
	OutMip->Init(Extent, Extent, SrcImage.NumSlices * 6, ERawImageFormat::RGBA32F, EGammaSpace::Linear);

	for (int32 Slice = 0; Slice < SrcImage.NumSlices; ++Slice)
	{
		FImageViewLongLat LongLatView(LongLatImage, Slice);

		// Parallel on the 6 faces :
		ParallelFor( TEXT("Texture.CubeMipFromLongLat.PF"),6,1, [&](int32 Face)
		{
			FImageView2D MipView(*OutMip, Slice * 6 + Face);
			for (uint32 y = 0; y < Extent; ++y)
			{
				for (uint32 x = 0; x < Extent; ++x)
				{
					FVector DirectionWS = ComputeWSCubeDirectionAtTexelCenter(Face, x, y, InvExtent);
					MipView.Access(x, y) = LongLatView.LookupLongLatDouble(DirectionWS);
				}
			}
		}, EParallelForFlags::Unbalanced);
	}
}

class FTexelProcessor
{
public:
	// @param InConeAxisSS - normalized, in side space
	// @param TexelAreaArray - precomputed area of each texel for correct weighting
	FTexelProcessor(const FVector& InConeAxisSS, float ConeAngle, const FLinearColor* InSideData, const float* InTexelAreaArray, uint32 InFullExtent)
		: ConeAxisSS(InConeAxisSS)
		, AccumulatedColor(0, 0, 0, 0)
		, SideData(InSideData)
		, TexelAreaArray(InTexelAreaArray)
		, FullExtent(InFullExtent)
	{
		ConeAngleSin = sinf(ConeAngle);
		ConeAngleCos = cosf(ConeAngle);

		// *2 as the position is from -1 to 1
		// / InFullExtent as x and y is in the range 0..InFullExtent-1
		PositionToWorldScale = 2.0f / (float)InFullExtent;
		InvFullExtent = 1.0f / (float)FullExtent;

		// examples: 0 to diffuse convolution, 0.95f for glossy
		DirDot = FMath::Min(FMath::Cos(ConeAngle), 0.9999f);

		InvDirOneMinusDot = 1.0f / (1.0f - DirDot);

		// precomputed sqrt(2.0f * 2.0f + 2.0f * 2.0f)
		float Sqrt8 = 2.8284271f;
		RadiusToWorldScale = Sqrt8 / (float)InFullExtent;
	}

	// @return true: yes, traverse deeper, false: not relevant
	bool TestIfRelevant(uint32 x, uint32 y, uint32 LocalExtent) const
	{
		float HalfExtent = (float)LocalExtent * 0.5f; 
		float U = ((float)x + HalfExtent) * PositionToWorldScale - 1.0f;
		float V = ((float)y + HalfExtent) * PositionToWorldScale - 1.0f;

		float SphereRadius = RadiusToWorldScale * (float)LocalExtent;

		FVector SpherePos(U, V, 1);

		return FMath::SphereConeIntersection(SpherePos, SphereRadius, ConeAxisSS, ConeAngleSin, ConeAngleCos);
	}

	void Process(uint32 x, uint32 y)
	{
		const FLinearColor* In = &SideData[x + y * FullExtent];
		
		FVector DirectionSS = ComputeSSCubeDirectionAtTexelCenter(x, y, InvFullExtent);

		float DotValue = static_cast<float>(ConeAxisSS | DirectionSS);

		if(DotValue > DirDot)
		{
			// 0..1, 0=at kernel border..1=at kernel center
			float KernelWeight = 1.0f - (1.0f - DotValue) * InvDirOneMinusDot;

			// apply smoothstep function (softer, less linear result)
			KernelWeight = KernelWeight * KernelWeight * (3 - 2 * KernelWeight);

			float AreaCompensation = TexelAreaArray[x + y * FullExtent];
			// AreaCompensation would be need for correctness but seems it has a but
			// as it looks much better (no seam) without, the effect is minor so it's deactivated for now.
//			float Weight = KernelWeight * AreaCompensation;
			float Weight = KernelWeight;

			AccumulatedColor.R += Weight * In->R;
			AccumulatedColor.G += Weight * In->G;
			AccumulatedColor.B += Weight * In->B;
			AccumulatedColor.A += Weight;
		}
	}

	// normalized, in side space
	FVector ConeAxisSS;

	FLinearColor AccumulatedColor;

	// cached for better performance
	float ConeAngleSin;
	float ConeAngleCos;
	float PositionToWorldScale;
	float RadiusToWorldScale;
	float InvFullExtent;
	// 0 to diffuse convolution, 0.95f for glossy
	float DirDot;
	float InvDirOneMinusDot;

	/** [x + y * FullExtent] */
	const FLinearColor* SideData;
	const float* TexelAreaArray;
	uint32 FullExtent;
};

template <class TVisitor>
void TCubemapSideRasterizer(TVisitor &TexelProcessor, int32 x, uint32 y, uint32 Extent)
{
	if(Extent > 1)
	{
		if(!TexelProcessor.TestIfRelevant(x, y, Extent))
		{
			return;
		}
		Extent /= 2;

		TCubemapSideRasterizer(TexelProcessor, x, y, Extent);
		TCubemapSideRasterizer(TexelProcessor, x + Extent, y, Extent);
		TCubemapSideRasterizer(TexelProcessor, x, y + Extent, Extent);
		TCubemapSideRasterizer(TexelProcessor, x + Extent, y + Extent, Extent);
	}
	else
	{
		TexelProcessor.Process(x, y);
	}
}

static FLinearColor IntegrateAngularArea(FImage& Image, FVector FilterDirectionWS, float ConeAngle, const float* TexelAreaArray)
{
	// Alpha channel is used to renormalize later
	FLinearColor ret(0, 0, 0, 0);
	int32 Extent = Image.SizeX;

	for(uint32 Face = 0; Face < 6; ++Face)
	{
		FImageView2D ImageView(Image, Face);
		FVector FilterDirectionSS = TransformWorldToSideSpace(Face, FilterDirectionWS);
		FTexelProcessor Processor(FilterDirectionSS, ConeAngle, &ImageView.Access(0,0), TexelAreaArray, Extent);

		// recursively split the (0,0)-(Extent-1,Extent-1), tests for intersection and processes only colors inside
		TCubemapSideRasterizer(Processor, 0, 0, Extent);
		ret += Processor.AccumulatedColor;
	}
	
	if(ret.A != 0)
	{
		float Inv = 1.0f / ret.A;

		ret.R *= Inv;
		ret.G *= Inv;
		ret.B *= Inv;
	}
	else
	{
		// should not happen
//		checkSlow(0);
	}

	ret.A = 0;

	return ret;
}

// @return 2 * computed triangle area 
static inline float TriangleArea2_3D(FVector A, FVector B, FVector C)
{
	return static_cast<float>(((A-B) ^ (C-B)).Size());
}

static inline float ComputeTexelArea(uint32 x, uint32 y, float InvSideExtentMul2)
{
	float fU = (float)x * InvSideExtentMul2 - 1;
	float fV = (float)y * InvSideExtentMul2 - 1;

	FVector CornerA = FVector(fU, fV, 1);
	FVector CornerB = FVector(fU + InvSideExtentMul2, fV, 1);
	FVector CornerC = FVector(fU, fV + InvSideExtentMul2, 1);
	FVector CornerD = FVector(fU + InvSideExtentMul2, fV + InvSideExtentMul2, 1);

	CornerA.Normalize();
	CornerB.Normalize();
	CornerC.Normalize();
	CornerD.Normalize();

	return TriangleArea2_3D(CornerA, CornerB, CornerC) + TriangleArea2_3D(CornerC, CornerB, CornerD) * 0.5f;
}

/**
 * Generate a mip using angular filtering.
 * @param DestMip - The filtered mip.
 * @param SrcMip - The source mip which will be filtered.
 * @param ConeAngle - The cone angle with which to filter.
 */
static void GenerateAngularFilteredMip(FImage* DestMip, FImage& SrcMip, float ConeAngle)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateAngularFilteredMip);

	int32 MipExtent = DestMip->SizeX;
	float MipInvSideExtent = 1.0f / (float)MipExtent;

	TArray<float> TexelAreaArray;
	TexelAreaArray.AddUninitialized(SrcMip.SizeX * SrcMip.SizeY);

	// precompute the area size for one face (is the same for each face)
	for(int32 y = 0; y < SrcMip.SizeY; ++y)
	{
		for(int32 x = 0; x < SrcMip.SizeX; ++x)
		{
			TexelAreaArray[x + y * (int64) SrcMip.SizeX] = ComputeTexelArea(x, y, MipInvSideExtent * 2);
		}
	}

	// We start getting gains running threaded upwards of sizes >= 128
	if (SrcMip.SizeX >= 128)
	{
		// Quick workaround: Do a thread per mip
		struct FAsyncGenerateMipsPerFaceWorker : public FNonAbandonableTask
		{
			int32 Face;
			FImage* DestMip;
			int32 Extent;
			float ConeAngle;
			const float* TexelAreaArray;
			FImage* SrcMip;
			FAsyncGenerateMipsPerFaceWorker(int32 InFace, FImage* InDestMip, int32 InExtent, float InConeAngle, const float* InTexelAreaArray, FImage* InSrcMip) :
				Face(InFace),
				DestMip(InDestMip),
				Extent(InExtent),
				ConeAngle(InConeAngle),
				TexelAreaArray(InTexelAreaArray),
				SrcMip(InSrcMip)
			{
			}

			void DoWork()
			{
				const float InvSideExtent = 1.0f / (float)Extent;
				FImageView2D DestMipView(*DestMip, Face);
				for (int32 y = 0; y < Extent; ++y)
				{
					for (int32 x = 0; x < Extent; ++x)
					{
						FVector DirectionWS = ComputeWSCubeDirectionAtTexelCenter(Face, x, y, InvSideExtent);
						DestMipView.Access(x, y) = IntegrateAngularArea(*SrcMip, DirectionWS, ConeAngle, TexelAreaArray);
					}
				}
			}

			FORCEINLINE TStatId GetStatId() const
			{
				RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncGenerateMipsPerFaceWorker, STATGROUP_ThreadPoolAsyncTasks);
			}
		};

		typedef FAsyncTask<FAsyncGenerateMipsPerFaceWorker> FAsyncGenerateMipsPerFaceTask;
		TIndirectArray<FAsyncGenerateMipsPerFaceTask> AsyncTasks;

		for (int32 Face = 0; Face < 6; ++Face)
		{
			auto* AsyncTask = new FAsyncGenerateMipsPerFaceTask(Face, DestMip, MipExtent, ConeAngle, TexelAreaArray.GetData(), &SrcMip);
			AsyncTasks.Add(AsyncTask);
			AsyncTask->StartBackgroundTask();
		}

		for (int32 TaskIndex = 0; TaskIndex < AsyncTasks.Num(); ++TaskIndex)
		{
			auto& AsyncTask = AsyncTasks[TaskIndex];
			AsyncTask.EnsureCompletion();
		}
	}
	else
	{
		for (int32 Face = 0; Face < 6; ++Face)
		{
			FImageView2D DestMipView(*DestMip, Face);
			for (int32 y = 0; y < MipExtent; ++y)
			{
				for (int32 x = 0; x < MipExtent; ++x)
				{
					FVector DirectionWS = ComputeWSCubeDirectionAtTexelCenter(Face, x, y, MipInvSideExtent);
					DestMipView.Access(x, y) = IntegrateAngularArea(SrcMip, DirectionWS, ConeAngle, TexelAreaArray.GetData());
				}
			}
		}
	}
}

void ITextureCompressorModule::GenerateAngularFilteredMips(TArray<FImage>& InOutMipChain, int32 NumMips, uint32 DiffuseConvolveMipLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.GenerateAngularFilteredMips);

	TArray<FImage> SrcMipChain;
	Exchange(SrcMipChain, InOutMipChain);
	InOutMipChain.Empty(NumMips);

	// note: should work on cube arrays but currently does not
	// GetMipGenSettings forces Angular off for anything but pure Cube classes (no arrays)
	check( SrcMipChain[0].NumSlices == 6 );

	// Generate simple averaged mips to accelerate angular filtering.
	for (int32 MipIndex = SrcMipChain.Num(); MipIndex < NumMips; ++MipIndex)
	{
		FImage& BaseMip = SrcMipChain[MipIndex - 1];
		int32 BaseExtent = BaseMip.SizeX;
		int32 MipExtent = FMath::Max(BaseExtent >> 1, 1);
		FImage& Mip = SrcMipChain.Emplace_GetRef(MipExtent, MipExtent, BaseMip.NumSlices, BaseMip.Format);

		for(int32 Face = 0; Face < 6; ++Face)
		{
			FImageView2D BaseMipView(BaseMip, Face);
			FImageView2D MipView(Mip, Face);

			for(int32 y = 0; y < MipExtent; ++y)
			{
				for(int32 x = 0; x < MipExtent; ++x)
				{		
					FLinearColor Sum = (
						BaseMipView.Access(x*2, y*2) +
						BaseMipView.Access(x*2+1, y*2) +
						BaseMipView.Access(x*2, y*2+1) +
						BaseMipView.Access(x*2+1, y*2+1)
						) * 0.25f;
					MipView.Access(x,y) = Sum;
				}
			}
		}
	}

	int32 Extent = 1 << (NumMips - 1);
	int32 BaseExtent = Extent;
	for (int32 i = 0; i < NumMips; ++i)
	{
		// 0:top mip 1:lowest mip = diffuse convolve
		float NormalizedMipLevel = (float)i / (float)(NumMips - DiffuseConvolveMipLevel);
		float AdjustedMipLevel = NormalizedMipLevel * (float)NumMips;
		float NormalizedWidth = (float)BaseExtent * FMath::Pow(2.0f, -AdjustedMipLevel);
		float TexelSize = 1.0f / NormalizedWidth;

		// 0.001f:sharp  .. PI/2: diffuse convolve
		// all lower mips are used for diffuse convolve
		// above that the angle blends from sharp to diffuse convolved version
		float ConeAngle = PI / 2.0f * TexelSize;

		// restrict to reasonable range
		ConeAngle = FMath::Clamp(ConeAngle, 0.002f, (float)PI / 2.0f);

		UE_LOGF(LogTextureCompressor, Verbose, "GenerateAngularFilteredMips  %f %f %f %f %f", NormalizedMipLevel, AdjustedMipLevel, NormalizedWidth, TexelSize, ConeAngle * 180 / PI);

		// 0:normal, -1:4x faster, +1:4 times slower but more precise, -2, 2 ...
		float QualityBias = 3.0f;

		// defined to result in a area of 1.0f (NormalizedArea)
		// optimized = 0.5f * FMath::Sqrt(1.0f / PI);
		float SphereRadius = 0.28209478f;
		float SegmentHeight = SphereRadius * (1.0f - FMath::Cos(ConeAngle));
		// compute SphereSegmentArea
		float AreaCoveredInNormalizedArea = 2 * PI * SphereRadius * SegmentHeight;
		checkSlow(AreaCoveredInNormalizedArea <= 0.5f);

		// unoptimized
		//	float FloatInputMip = FMath::Log2(FMath::Sqrt(AreaCoveredInNormalizedArea)) + InputMipCount - QualityBias;
		// optimized
		float FloatInputMip = 0.5f * FMath::Log2(AreaCoveredInNormalizedArea) + (float)NumMips - QualityBias;
		uint32 InputMip = FMath::Clamp(FMath::TruncToInt(FloatInputMip), 0, NumMips - 1);

		FImage& Mip = InOutMipChain.Emplace_GetRef(Extent, Extent, 6, ERawImageFormat::RGBA32F);
		GenerateAngularFilteredMip(&Mip, SrcMipChain[InputMip], ConeAngle);
		Extent = FMath::Max(Extent >> 1, 1);
	}
}

static bool NeedAdjustImageColors(const FTextureBuildSettings& InBuildSettings)
{
	const FColorAdjustmentParameters& InParams = InBuildSettings.ColorAdjustment;

	return
		!FMath::IsNearlyEqual(InParams.AdjustBrightness, 1.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustBrightnessCurve, 1.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustSaturation, 1.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustVibrance, 0.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustRGBCurve, 1.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustHue, 0.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustMinAlpha, 0.0f, (float)KINDA_SMALL_NUMBER) ||
		!FMath::IsNearlyEqual(InParams.AdjustMaxAlpha, 1.0f, (float)KINDA_SMALL_NUMBER) ||
		InBuildSettings.bChromaKeyTexture;
}

static inline void AdjustColorsOld(FLinearColor * Colors,int64 Count, const FTextureBuildSettings& InBuildSettings)
{
	const FColorAdjustmentParameters& Params = InBuildSettings.ColorAdjustment;
	
	// very similar to AdjustColorsNew
	// issues in here are mostly fixed in AdjustColorsNew

	// note: : not the same checks as bAdjustNeeded outside
	//	 but preserves legacy behavior
	bool bAdjustBrightnessCurve = (!FMath::IsNearlyEqual(Params.AdjustBrightnessCurve, 1.0f, (float)KINDA_SMALL_NUMBER) && Params.AdjustBrightnessCurve != 0.0f);
	bool bAdjustVibrance = (!FMath::IsNearlyZero(Params.AdjustVibrance, (float)KINDA_SMALL_NUMBER));
	bool bAdjustRGBCurve = (!FMath::IsNearlyEqual(Params.AdjustRGBCurve, 1.0f, (float)KINDA_SMALL_NUMBER) && Params.AdjustRGBCurve != 0.0f);
	bool bAdjustSaturation = ( Params.AdjustSaturation != 1.f || bAdjustVibrance );
	bool bAdjustValue = ( Params.AdjustBrightness != 1.f ) || bAdjustBrightnessCurve;

	float AdjustHue = Params.AdjustHue;
	if ( AdjustHue != 0.f )
	{
		// Params.AdjustHue should be in [0,360] , make sure
		if ( AdjustHue < 0.f || AdjustHue > 360.f )
		{
			AdjustHue = fmodf(AdjustHue, 360.0f);
			if ( AdjustHue < 0.f )
			{
				AdjustHue += 360.f;
			}
		}
	}
	
	// BuildSettings.ChromaKeyColor is an FColor
	FLinearColor ChromaKeyColor(InBuildSettings.ChromaKeyColor);

	bool bHDRSource = InBuildSettings.bHDRSource;
	bool bChromaKeyTexture = InBuildSettings.bChromaKeyTexture;
	float ChromaKeyThreshold = InBuildSettings.ChromaKeyThreshold + SMALL_NUMBER;

	for(int64 i=0;i<Count;i++)
	{
		FLinearColor OriginalColor = Colors[i];
	
		// note: non-HDR source data in [0,1] can drift outside of [0,1]
		//   and then bad things will happen here
		//  left alone to preserve old behavior

		if (bChromaKeyTexture && (OriginalColor.Equals(ChromaKeyColor, ChromaKeyThreshold)))
		{
			OriginalColor = FLinearColor::Transparent;

			//note: strange: no return.  processing continues on the transparent color...
			//	  this was likely unintentional
			//Colors[i] = FLinearColor::Transparent;
			//continue;
		}

		// NOTE: if OriginalColor has HDR/floats in it, this function does not handle it well
		//	it implicitly discards negatives (and negatives cause color shifts)
		//	for values > 1 the clamp behavior is very strange

		// Convert to HSV
		FLinearColor HSVColor = OriginalColor.LinearRGBToHSV();
		float& PixelHue = HSVColor.R;
		float& PixelSaturation = HSVColor.G;
		float& PixelValue = HSVColor.B;

		float OriginalLuminance = PixelValue;

		if ( bAdjustValue )
		{
			// Apply brightness adjustment
			PixelValue *= Params.AdjustBrightness;

			// Apply brightness power adjustment
			if ( bAdjustBrightnessCurve )
			{
				// Raise HSV.V to the specified power
				PixelValue = FMath::Pow(PixelValue, Params.AdjustBrightnessCurve);
			}
		
			// Clamp brightness if non-HDR
			if (!bHDRSource)
			{
				PixelValue = FMath::Clamp(PixelValue, 0.0f, 1.0f);
			}
		}
		
		if ( bAdjustSaturation )
		{
			// PixelSaturation is >= 0 but not <= 1
			//  because negative RGB can come into this function which gives Saturation > 1

			// Apply "vibrance" adjustment
			if ( bAdjustVibrance )
			{
				// note: AdjustVibrance is disabled for HDR source in the Texture UPROPERTIES
				//    (unclear why, this is no worse than anything else here on HDR)
				const float SatRaisePow = 5.0f;
				const float InvSatRaised = FMath::Pow(1.0f - PixelSaturation, SatRaisePow);

				const float ClampedVibrance = FMath::Clamp(Params.AdjustVibrance, 0.0f, 1.0f);
				const float HalfVibrance = ClampedVibrance * 0.5f;

				const float SatProduct = HalfVibrance * InvSatRaised;

				PixelSaturation += SatProduct;
			}

			// Apply saturation adjustment
			PixelSaturation *= Params.AdjustSaturation;
			PixelSaturation = FMath::Clamp(PixelSaturation, 0.0f, 1.0f);
		}

		// Apply hue adjustment
		if ( AdjustHue != 0.f )
		{
			// PixelHue is [0,360) but AdjustHue is [0,360]
			PixelHue += AdjustHue;

			// Clamp HSV values
			if ( PixelHue >= 360.f )
			{
				PixelHue -= 360.f;
			}
		}

		// Convert back to a linear color
		FLinearColor LinearColor = HSVColor.HSVToLinearRGB();

		// Apply RGB curve adjustment (linear space)
		if ( bAdjustRGBCurve )
		{
			LinearColor.R = FMath::Pow(LinearColor.R, Params.AdjustRGBCurve);
			LinearColor.G = FMath::Pow(LinearColor.G, Params.AdjustRGBCurve);
			LinearColor.B = FMath::Pow(LinearColor.B, Params.AdjustRGBCurve);
		}

		// Clamp HDR RGB channels to 1 or the original luminance (max original RGB channel value), whichever is greater
		// note: this is a very odd thing to do
		//		clamping at OriginalLuminance if you do AdjustBrightness or AdjustBrightnessCurve ?
		//	    that would keep values brighter than 1.f unchanged, but bring up lower ones to 1.f
		if (bHDRSource)
		{
			LinearColor.R = FMath::Clamp(LinearColor.R, 0.0f, (OriginalLuminance > 1.0f ? OriginalLuminance : 1.0f));
			LinearColor.G = FMath::Clamp(LinearColor.G, 0.0f, (OriginalLuminance > 1.0f ? OriginalLuminance : 1.0f));
			LinearColor.B = FMath::Clamp(LinearColor.B, 0.0f, (OriginalLuminance > 1.0f ? OriginalLuminance : 1.0f));
		}

		// Remap the alpha channel
		LinearColor.A = FMath::Lerp(Params.AdjustMinAlpha, Params.AdjustMaxAlpha, OriginalColor.A);

		Colors[i] = LinearColor;
	}
}

// see also AdjustColorsOld
static inline void AdjustColorsNew(FLinearColor* Colors, int64 Count, const FTextureBuildSettings& InBuildSettings)
{
	const FColorAdjustmentParameters& Params = InBuildSettings.ColorAdjustment;

	// @todo Oodle : not the same checks as bAdjustNeeded outside
	//	 but preserves legacy behavior
	bool bAdjustBrightnessCurve = (!FMath::IsNearlyEqual(Params.AdjustBrightnessCurve, 1.0f, (float)KINDA_SMALL_NUMBER) && Params.AdjustBrightnessCurve != 0.0f);
	bool bAdjustVibrance = (!FMath::IsNearlyZero(Params.AdjustVibrance, (float)KINDA_SMALL_NUMBER));
	bool bAdjustRGBCurve = (!FMath::IsNearlyEqual(Params.AdjustRGBCurve, 1.0f, (float)KINDA_SMALL_NUMBER) && Params.AdjustRGBCurve != 0.0f);
	bool bAdjustSaturation = ( Params.AdjustSaturation != 1.f || bAdjustVibrance );
	bool bAdjustValue = ( Params.AdjustBrightness != 1.f ) || bAdjustBrightnessCurve;

	float AdjustHue = Params.AdjustHue;
	if ( AdjustHue != 0.f )
	{
		// Params.AdjustHue should be in [0,360] , make sure
		if ( AdjustHue < 0.f || AdjustHue > 360.f )
		{
			AdjustHue = fmodf(AdjustHue, 360.0f);
			if ( AdjustHue < 0.f )
			{
				AdjustHue += 360.f;
			}
		}
	}
	
	// BuildSettings.ChromaKeyColor is an FColor
	FLinearColor ChromaKeyColor(InBuildSettings.ChromaKeyColor);
	bool bChromaKeyTexture = InBuildSettings.bChromaKeyTexture;
	float ChromaKeyThreshold = InBuildSettings.ChromaKeyThreshold + SMALL_NUMBER;

	bool bHDRSource = InBuildSettings.bHDRSource;

	for (int64 i=0; i<Count; i++)
	{
		if (bChromaKeyTexture && (Colors[i].Equals(ChromaKeyColor, ChromaKeyThreshold)))
		{
			Colors[i] = FLinearColor::Transparent;
			continue;
		}

		FLinearColor OriginalColor = Colors[i];
			
		if (!bHDRSource)
		{
			// Ensure we are clamped as expected (can drift out of clamp due to previous processing)
			// if you wind up even very slightly out of [0,1] range this function does bad things
			OriginalColor.R = FMath::Clamp(OriginalColor.R, 0.0f, 1.f);
			OriginalColor.G = FMath::Clamp(OriginalColor.G, 0.0f, 1.f);
			OriginalColor.B = FMath::Clamp(OriginalColor.B, 0.0f, 1.f);
			// A will be clamped later when used
		}
		else
		{			
			// yes HDR source, but we don't support negatives in Adjust
			// clamp to >= 0 :
			OriginalColor.R = FMath::Max(OriginalColor.R, 0.0f);
			OriginalColor.G = FMath::Max(OriginalColor.G, 0.0f);
			OriginalColor.B = FMath::Max(OriginalColor.B, 0.0f);
		}

		// Convert to HSV
		const FLinearColor OriginalHSVColor = OriginalColor.LinearRGBToHSV();
		float PixelHue = OriginalHSVColor.R;
		float PixelSaturation = OriginalHSVColor.G;
		float PixelValue = OriginalHSVColor.B;

		if ( bAdjustValue )
		{
			// Apply brightness adjustment
			PixelValue *= Params.AdjustBrightness;

			// Apply brightness power adjustment
			if ( bAdjustBrightnessCurve )
			{
				// Raise HSV.V to the specified power
				PixelValue = FMath::Pow(PixelValue, Params.AdjustBrightnessCurve);
			}
		
			// Clamp brightness if non-HDR
			if (!bHDRSource)
			{
				PixelValue = FMath::Clamp(PixelValue, 0.0f, 1.0f);
			}
		}
		
		if ( bAdjustSaturation )
		{
			// PixelSaturation is >= 0 but not <= 1
			//  because negative RGB can come into this function which gives Saturation > 1

			// Apply "vibrance" adjustment
			if ( bAdjustVibrance )
			{
				const float InvSat = 1.0f - PixelSaturation;
				const float InvSatRaised = InvSat * InvSat * InvSat * InvSat * InvSat;

				const float ClampedVibrance = FMath::Clamp(Params.AdjustVibrance, 0.0f, 1.0f);
				const float HalfVibrance = ClampedVibrance * 0.5f;

				const float SatProduct = HalfVibrance * InvSatRaised;

				PixelSaturation += SatProduct;
			}

			// Apply saturation adjustment
			PixelSaturation *= Params.AdjustSaturation;
		}

		// Apply hue adjustment
		// PixelHue is [0,360) but AdjustHue is [0,360]
		PixelHue += AdjustHue;

		// Convert back to a linear color
		FLinearColor HSVColor(PixelHue,PixelSaturation,PixelValue);
		// HSVToLinearRGB does the clamping/fmod for you
		FLinearColor LinearColor = HSVColor.HSVToLinearRGB();

		// Apply RGB curve adjustment (linear space)
		if ( bAdjustRGBCurve )
		{
			LinearColor.R = FMath::Pow(LinearColor.R, Params.AdjustRGBCurve);
			LinearColor.G = FMath::Pow(LinearColor.G, Params.AdjustRGBCurve);
			LinearColor.B = FMath::Pow(LinearColor.B, Params.AdjustRGBCurve);
		}

		// Remap the alpha channel
		LinearColor.A = FMath::Lerp(Params.AdjustMinAlpha, Params.AdjustMaxAlpha, FMath::Clamp(OriginalColor.A, 0.f, 1.f));

		Colors[i] = LinearColor;
	}
}

void ITextureCompressorModule::AdjustImageColors(FImage& Image, const FTextureBuildSettings& InBuildSettings)
{
	const FColorAdjustmentParameters& InParams = InBuildSettings.ColorAdjustment;
	check( Image.SizeX > 0 && Image.SizeY > 0 );

	// @todo Oodle : this bAdjustNeeded is not checking the same conditions to enable these adjustments
	//		as is used inside the AdjustColors() routine
	//		this is how it was done in the past, so keep it the same to preserve legacy operation
	//	if possible in the future factor this Needed check out so it is shared code

	bool bAdjustNeeded = NeedAdjustImageColors(InBuildSettings);

	if ( bAdjustNeeded )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.AdjustImageColors);

		FImageCore::ImageParallelFor( TEXT("Texture.AdjustImageColorsFunc.PF"),Image, [&](FImageView & ImagePart,int64 RowY)
		{
			TArrayView64<FLinearColor> ImageColors = ImagePart.AsRGBA32F();

			FLinearColor * First = &ImageColors[0];
			int64 Count = ImageColors.Num();

			// Use new AdjustColors code only for newly added textures
			// So existing textures maintain exactly same output as before
			if (InBuildSettings.bUseNewMipFilter)
			{
				AdjustColorsNew(First, Count, InBuildSettings);
			}
			else
			{
				AdjustColorsOld(First, Count, InBuildSettings);
			}
		});
	}
}

// if alpha cannot be determined, returns false and does not fill bOutAlphaIsTransparent
// if possible, returns true, fills out bOutAlphaIsTransparent
bool ITextureCompressorModule::DetermineAlphaChannelTransparency(const FTextureBuildSettings& InBuildSettings, const FLinearColor& InChannelMin, const FLinearColor& InChannelMax, bool& bOutAlphaIsTransparent)
{
	// Settings that affect alpha:
	// 1. ColorTransforms - if they have a color transform we assume it's not affecting alpha (this is the UE
	//		integration assumption separate from this) this only gets dicey if we have ReplicateRed. Probably
	//		the best thing to do is just assume the transform doesn't affects opaqueness and let them use the
	//		force overrides if it's wrong.
	// 2. ForceAlphaChannel - Note ForceNoAlpha forces BC1 before us so we don't see it.
	// 3. AdjustMinAlpha, AdjustMaxAlpha
	// 4. ReplicateRed.. which means we need to track all operations on the red channel. Can probably just call
	//		AdjustImageColors directly on our boundaries and see where they go?
	// 5. ChromaKey! I think if they have this on we assume it matches SOMEWHERE and we need alpha.

	// ForceNo gets applied first because it happens at the texture format selection phase, so it effectively
	// overrides everything as AutoDXT gets cleared.
	if (InBuildSettings.bForceNoAlphaChannel)
	{
		bOutAlphaIsTransparent = false; // note we don't see this with autodxt as it gets forced to BC1 early.
		return true;
	}

	if (InBuildSettings.bForceAlphaChannel ||
		InBuildSettings.bChromaKeyTexture // If they have a chroma key they are expecting transparency!
		)
	{
		bOutAlphaIsTransparent = true;
		return true;
	}

	// No forces - try and predict how the color transforms will affect the colors.
	if (InBuildSettings.bHasColorSpaceDefinition)
	{
		// We assert that color spaces don't affect alpha per our integration with OCIO. So, the only way 
		// this affects us is if RepliateRed is set. 
		// 
		// We assume the color space doesn't change the opaquenss of the red channel so we can pass through to
		// the normal replicate red behavior. This could definitely fail, but we don't really expect anyone
		// to combine a color space remap _and_ replicate red, so if they hit this they can get what they want
		// with Force/ForceNoAlpha.
		if (InBuildSettings.bReplicateRed)
		{
			return false;
		}
	}

	if (InBuildSettings.bComputeBokehAlpha)
	{
		// Bokeh stores information in the alpha channel during processing - so we need an alpha channel
		bOutAlphaIsTransparent = true;
		return true;
	}

	FLinearColor ChannelMinMax[2] = {InChannelMin, InChannelMax};

	// If there's an image adjustment, run it on the colors so we see what happens.
	// This also handles AdjustMinAlpha/MaxAlpha.
	if (NeedAdjustImageColors(InBuildSettings))
	{
		if (InBuildSettings.bUseNewMipFilter)
		{
			AdjustColorsNew(ChannelMinMax, 2, InBuildSettings);
		}
		else
		{
			AdjustColorsOld(ChannelMinMax, 2, InBuildSettings);
		}
	}

	if (InBuildSettings.bReplicateRed)
	{
		ChannelMinMax[0].A = ChannelMinMax[0].R;
		ChannelMinMax[1].A = ChannelMinMax[1].R;
	}
	
	// use the same test as FImageCore::DetectAlphaChannel
	// @todo : this is NOT the same threshold used for RGBA16 in DetectAlphaChannel
	const float FloatNonOpaqueAlpha = 254.5f / 255.f; // the U8 alpha threshold

	float MinAlpha = FMath::Min(ChannelMinMax[0].A ,ChannelMinMax[1].A);

	bOutAlphaIsTransparent = ( MinAlpha <= FloatNonOpaqueAlpha );
	
	// detect ambiguity zone where alpha could easily cross the threshold due to float math :
	// Removed -- Any time this happens the author can't have predicted what alpha would be so
	// use whatever result we select to be consistent. This is not hit in any known texture.
	//if ( MinAlpha >= (254.4f/255.f) && MinAlpha <= (254.6f/255.f) )
	//{
	//	return false; // not possible to answer bOutAlphaIsTransparent
	//}

	return true;
}

/**
 * Compute the alpha channel how BokehDOF needs it setup
 *
 * @param	Image	Image to adjust
 */
static void ComputeBokehAlpha(FImage& Image)
{
	check( Image.SizeX > 0 && Image.SizeY > 0 );

	// compute LinearAverage
	FLinearColor LinearAverage = FImageCore::ComputeImageLinearAverage(Image);

	FLinearColor Scale(1, 1, 1, 1);

	// we want to normalize the image to have 0.5 as average luminance, this is assuming clamping doesn't happen (can happen when using a very small Bokeh shape)
	{
		float RGBLum = (LinearAverage.R + LinearAverage.G + LinearAverage.B) * (1.f/3.0f);

		// ideally this would be 1 but then some pixels would need to be >1 which is not supported for the textureformat we want to use.
		// The value affects the occlusion computation of the BokehDOF
		const float LumGoal = 0.25f;

		// clamp to avoid division by 0
		Scale *= LumGoal / FMath::Max(RGBLum, 0.001f);
	}

	FImageCore::ImageParallelProcessLinearPixels(TEXT("PF.ComputeBokehAlpha"),Image,[&](TArrayView64<FLinearColor> & Colors,int64 RowY) {
		for( FLinearColor & Color : Colors )
		{
			// Convert to a linear color
			FLinearColor LinearColor = Color * Scale;
			float RGBLum = (LinearColor.R + LinearColor.G + LinearColor.B) * (1.f/3.0f);
			LinearColor.A = FMath::Clamp(RGBLum, 0.0f, 1.0f);
			Color = LinearColor;
		}
		return FImageCore::ProcessLinearPixelsAction::Modified;
	} );
}

/**
 * Replicates the contents of the red channel to the green, blue, and alpha channels.
 */
static void ReplicateRedChannel( TArray<FImage>& InOutMipChain )
{
	const uint32 MipCount = InOutMipChain.Num();
	for ( uint32 MipIndex = 0; MipIndex < MipCount; ++MipIndex )
	{
		FImage& SrcMip = InOutMipChain[MipIndex];
		FLinearColor* FirstColor = (&SrcMip.AsRGBA32F()[0]);
		FLinearColor* LastColor = FirstColor + SrcMip.GetNumPixels();
		for ( FLinearColor* Color = FirstColor; Color < LastColor; ++Color )
		{
			*Color = FLinearColor( Color->R, Color->R, Color->R, Color->R );
		}
	}
}

/**
 * Replicates the contents of the alpha channel to the red, green, and blue channels.
 */
static void ReplicateAlphaChannel( TArray<FImage>& InOutMipChain )
{
	const uint32 MipCount = InOutMipChain.Num();
	for ( uint32 MipIndex = 0; MipIndex < MipCount; ++MipIndex )
	{
		FImage& SrcMip = InOutMipChain[MipIndex];
		FLinearColor* FirstColor = (&SrcMip.AsRGBA32F()[0]);
		FLinearColor* LastColor = FirstColor + SrcMip.GetNumPixels();
		for ( FLinearColor* Color = FirstColor; Color < LastColor; ++Color )
		{
			*Color = FLinearColor( Color->A, Color->A, Color->A, Color->A );
		}
	}
}

/**
 * Flips the contents of the green channel.
 * @param InOutMipChain - The mip chain on which the green channel shall be flipped.
 */
static void FlipGreenChannel( FImage& Image )
{
	FLinearColor* FirstColor = (&Image.AsRGBA32F()[0]);
	FLinearColor* LastColor = FirstColor + (Image.SizeX * Image.SizeY * Image.NumSlices);
	for ( FLinearColor* Color = FirstColor; Color < LastColor; ++Color )
	{
		Color->G = 1.0f - FMath::Clamp(Color->G, 0.0f, 1.0f);
	}
}

/** Calculate a scale per 4x4 block of each image, and apply it to the red/green channels. Store scale in the blue channel. */
static void ApplyYCoCgBlockScale(TArray<FImage>& InOutMipChain)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.ApplyYCoCgBlockScale);

	const uint32 MipCount = InOutMipChain.Num();
	for (uint32 MipIndex = 0; MipIndex < MipCount; ++MipIndex)
	{
		FImage& SrcMip = InOutMipChain[MipIndex];
		FLinearColor* FirstColor = (&SrcMip.AsRGBA32F()[0]);

		int32 BlockWidthX = SrcMip.SizeX / 4;
		int32 BlockWidthY = SrcMip.SizeY / 4;

		for (int32 Slice = 0; Slice < SrcMip.NumSlices; ++Slice)
		{
			FLinearColor* SliceFirstColor = FirstColor + SrcMip.GetSliceNumPixels() * Slice;

			for (int32 Y = 0; Y < BlockWidthY; ++Y)
			{
				FLinearColor* RowFirstColor = SliceFirstColor + Y * 4 * (int64) SrcMip.SizeX;

				for (int32 X = 0; X < BlockWidthX; ++X)
				{
					FLinearColor* BlockFirstColor = RowFirstColor + (X * 4);

					// Iterate block to find MaxComponent
					float MaxComponent = 0.f;
					for (int32 BlockY = 0; BlockY < 4; ++BlockY)
					{
						FLinearColor* Color = BlockFirstColor + BlockY * SrcMip.SizeX;
						for (int32 BlockX = 0; BlockX < 4; ++BlockX, ++Color)
						{
							MaxComponent = FMath::Max(FMath::Abs(Color->R - 128.f / 255.f), MaxComponent);
							MaxComponent = FMath::Max(FMath::Abs(Color->G - 128.f / 255.f), MaxComponent);
						}
					}

					const float Scale = (MaxComponent < 32.f / 255.f) ? 4.f : (MaxComponent < 64.f / 255.f) ? 2.f : 1.f;
					const float OutB = (Scale - 1.f) * 8.f / 255.f;

					// Iterate block to modify for scale
					for (int32 BlockY = 0; BlockY < 4; ++BlockY)
					{
						FLinearColor* Color = BlockFirstColor + BlockY * SrcMip.SizeX;
						for (int32 BlockX = 0; BlockX < 4; ++BlockX, ++Color)
						{
							const float OutR = (Color->R - 128.f / 255.f) * Scale + 128.f / 255.f;
							const float OutG = (Color->G - 128.f / 255.f) * Scale + 128.f / 255.f;

							*Color = FLinearColor(OutR, OutG, OutB, Color->A);
						}
					}
				}
			}
		}
	}
}

#if 0
static float RoughnessToSpecularPower(float Roughness)
{
	float Div = FMath::Pow(Roughness, 4);

	// Roughness of 0 should result in a high specular power
	float MaxSpecPower = 10000000000.0f;
	Div = FMath::Max(Div, 2.0f / (MaxSpecPower + 2.0f));

	return 2.0f / Div - 2.0f;
}

static float SpecularPowerToRoughness(float SpecularPower)
{
	float Out = FMath::Pow( SpecularPower * 0.5f + 1.0f, -0.25f );

	return Out;
}
#endif

static float CompositeNormalLengthToRoughness(const float LengthN, float Roughness, float CompositePower)
{
	float Variance = ( 1.0f - LengthN ) / LengthN;
	Variance = Variance - 0.00004f;
	if ( Variance <= 0.f )
	{
		return Roughness;
	}

	Variance *= CompositePower;
		
#if 0
	float Power = RoughnessToSpecularPower( Roughness );
	Power = Power / ( 1.0f + Variance * Power );
	Roughness = SpecularPowerToRoughness( Power );
#else
	// Refactored above to avoid divide by zero
	float a = Roughness * Roughness;
	float a2 = a * a;
	float B = 2.0f * Variance * (a2 - 1.0f);
	a2 = ( B - a2 ) / ( B - 1.0f );
	Roughness = FMath::Pow( a2, 0.25f );
#endif

	return Roughness;
}

static float CompositeNormalToRoughness(const FLinearColor & NormalColor, float Roughness, float CompositePower)
{
	FVector3f Normal = FVector3f(NormalColor.R * 2.0f - 1.0f, NormalColor.G * 2.0f - 1.0f, NormalColor.B * 2.0f - 1.0f);
				
	// Toksvig estimation of variance
	float LengthN = FMath::Min( Normal.Size(), 1.0f );

	return CompositeNormalLengthToRoughness(LengthN,Roughness,CompositePower);
}

// @param CompositeTextureMode original type ECompositeTextureMode
//	write roughness to specified channel of DestRoughness, computed from SourceNormals
static bool ApplyCompositeTexture(FImage& DestRoughness, const FImage& SourceNormals, uint8 CompositeTextureMode, float CompositePower)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.ApplyCompositeTexture);

	FLinearColor* FirstColor = (&DestRoughness.AsRGBA32F()[0]);
		
	float* TargetValuePtr;

	switch((ECompositeTextureMode)CompositeTextureMode)
	{
		case CTM_NormalRoughnessToRed:
			TargetValuePtr = &FirstColor->R;
			break;
		case CTM_NormalRoughnessToGreen:
			TargetValuePtr = &FirstColor->G;
			break;
		case CTM_NormalRoughnessToBlue:
			TargetValuePtr = &FirstColor->B;
			break;
		case CTM_NormalRoughnessToAlpha:
			TargetValuePtr = &FirstColor->A;
			break;
		default:
			UE_LOGF(LogTextureCompressor, Error, "Invalid CompositeTextureMode");
			return false;
	}
	
	if ( DestRoughness.SizeX == SourceNormals.SizeX && DestRoughness.SizeY == SourceNormals.SizeY &&
		DestRoughness.NumSlices == SourceNormals.NumSlices )
	{
		int64 Count = (int64) DestRoughness.SizeX * DestRoughness.SizeY * DestRoughness.NumSlices;

		const FLinearColor* NormalColors = (&SourceNormals.AsRGBA32F()[0]);

		for ( int64 i=0; i<Count; i++ )
		{
			const FLinearColor & NormalColor = NormalColors[i];

			float Roughness = TargetValuePtr[i*4];
			TargetValuePtr[i*4] = CompositeNormalToRoughness(NormalColor,Roughness,CompositePower);
		}
	}
	else
	{
		// sizes don't match, stretch with bilinear filter (filter the normal lengths, not the normal colors)

		if ( SourceNormals.NumSlices != 1 ||
			 DestRoughness.NumSlices != 1 )
		{
			UE_LOGF(LogTextureCompressor, Warning, "CompositeTexture XY sizes don't match, stretch not supported because slice count is not 1 : %d vs %d",
				SourceNormals.NumSlices,DestRoughness.NumSlices);
			return false;
		}

		TArray64<float> SourceNormalLengths;

		{
			int64 SourceNormalCount = (int64) SourceNormals.SizeX * SourceNormals.SizeY;
			SourceNormalLengths.SetNum(SourceNormalCount);

			const FLinearColor* NormalColors = (&SourceNormals.AsRGBA32F()[0]);

			for ( int64 i=0; i<SourceNormalCount; i++ )
			{
				const FLinearColor & NormalColor = NormalColors[i];
				
				FVector3f Normal = FVector3f(NormalColor.R * 2.0f - 1.0f, NormalColor.G * 2.0f - 1.0f, NormalColor.B * 2.0f - 1.0f);
				
				float LengthN = FMath::Min( Normal.Size(), 1.0f );

				SourceNormalLengths[i] = LengthN;
			}
		}

		//const FImageView2D NormalColors(const_cast<FImage &>(SourceNormals),0);
		const float * SourceNormalLengthsPlane = &SourceNormalLengths[0];

		float InvDestW = 1.f/(float)DestRoughness.SizeX;
		float InvDestH = 1.f/(float)DestRoughness.SizeY;

		for( int64 DestY=0; DestY< DestRoughness.SizeY; DestY++)
		{
			float V = ((float)DestY + 0.5f) * InvDestH;
			for( int64 DestX=0; DestX< DestRoughness.SizeX; DestX++)
			{
				float U = ((float)DestX + 0.5f) * InvDestW;
				
				//const FLinearColor NormalColor = LookupSourceMipBilinearUV(NormalColors,U,V);
				const float NormalLength = LookupFloatBilinearUV(SourceNormalLengthsPlane,SourceNormals.SizeX,SourceNormals.SizeY,U,V);

				float Roughness = *TargetValuePtr;
				*TargetValuePtr = CompositeNormalLengthToRoughness(NormalLength,Roughness,CompositePower);
				TargetValuePtr += 4;
			}
		}
	}

	return true;
}

/*------------------------------------------------------------------------------
	Image Compression.
------------------------------------------------------------------------------*/

void FTextureBuildSettings::GetEncodedTextureDescriptionWithPixelFormat(FEncodedTextureDescription* OutTextureDescription, EPixelFormat InEncodedPixelFormat, int32 InEncodedMip0SizeX, int32 InEncodedMip0SizeY, int32 InEncodedMip0NumSlices, int32 InMipCount) const
{
	FEncodedTextureDescription& TextureDescription = *OutTextureDescription;
	TextureDescription = FEncodedTextureDescription();
	TextureDescription.bCubeMap = bCubemap;
	TextureDescription.bTextureArray = bTextureArray;
	TextureDescription.bVolumeTexture = bVolume;
	TextureDescription.NumMips = IntCastChecked<uint8>(InMipCount);
	TextureDescription.PixelFormat = InEncodedPixelFormat;

	TextureDescription.TopMipSizeX = InEncodedMip0SizeX;
	TextureDescription.TopMipSizeY = InEncodedMip0SizeY;
	TextureDescription.TopMipVolumeSizeZ = bVolume ? InEncodedMip0NumSlices : 1;
	if (bTextureArray)
	{
		if (bCubemap)
		{
			TextureDescription.ArraySlices = InEncodedMip0NumSlices / 6;
		}
		else
		{
			TextureDescription.ArraySlices = InEncodedMip0NumSlices;
		}
	}
	else
	{
		TextureDescription.ArraySlices = 1;
	}
}

uint32 FTextureBuildSettings::GetOpenColorIOVersion()
{
	return OpenColorIOWrapper::GetVersionHex();
}

void FTextureBuildSettings::GetEncodedTextureDescription(FEncodedTextureDescription* OutTextureDescription, const ITextureFormat* InTextureFormat, int32 InEncodedMip0SizeX, int32 InEncodedMip0SizeY, int32 InEncodedMip0NumSlices, int32 InMipCount, bool bInImageHasAlphaChannel) const
{
	EPixelFormat EncodedPixelFormat = InTextureFormat->GetEncodedPixelFormat(*this, bInImageHasAlphaChannel);
	GetEncodedTextureDescriptionWithPixelFormat(OutTextureDescription, EncodedPixelFormat, InEncodedMip0SizeX, InEncodedMip0SizeY, InEncodedMip0NumSlices, InMipCount);
}

bool FTextureBuildSettings::GetEncodedTextureDescriptionFromSourceMips(
	FEncodedTextureDescription* OutTextureDescription, const ITextureFormat* InTextureFormat, 
	int32 InSourceMip0SizeX, int32 InSourceMip0SizeY, int32 InSourceMip0NumSlices, int32 InSourceMipCount, 
	bool bInImageHasAlphaChannel) const
{
	int32 EncodedSizeX, EncodedSizeY, EncodedNumSlices, EncodedMipCount;
	if (!GetOutputMipInfo(
		InSourceMip0SizeX, InSourceMip0SizeY, InSourceMip0NumSlices, InSourceMipCount, 
		EncodedSizeX, EncodedSizeY, EncodedNumSlices, EncodedMipCount))
	{
		return false;
	}

	EPixelFormat EncodedPixelFormat = InTextureFormat->GetEncodedPixelFormat(*this, bInImageHasAlphaChannel);
	GetEncodedTextureDescriptionWithPixelFormat(OutTextureDescription, EncodedPixelFormat, EncodedSizeX, EncodedSizeY, EncodedNumSlices, EncodedMipCount);
	return true;
}

// compress mip-maps in InMipChain and add mips to Texture, might alter the source content
static bool CompressMipChain(
	const ITextureFormat* TextureFormat,
	TArray<FImage>& MipChain,
	const FTextureBuildSettings& Settings,
	const bool bImageHasAlphaChannel, // bImageHasAlphaChannel might be unknown and invalid, however we know that if it matters to the pixel format it IS known.
	FStringView DebugTexturePathName,
	TArray<FCompressedImage2D>& OutMips,
	uint32& OutNumMipsInTail,
	uint32& OutExtData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CompressMipChain)

	// Determine ExtData (platform specific data) and NumMipsInTail.
	FEncodedTextureExtendedData ExtendedData;
	FEncodedTextureDescription TextureDescription;
	Settings.GetEncodedTextureDescription(&TextureDescription, TextureFormat, MipChain[0].SizeX, MipChain[0].SizeY, MipChain[0].NumSlices, MipChain.Num(), bImageHasAlphaChannel);
	{
		ExtendedData = TextureFormat->GetExtendedDataForTexture(TextureDescription, Settings.LODBias);
		OutNumMipsInTail = ExtendedData.NumMipsInTail;
		OutExtData = ExtendedData.ExtData;
	}

	int32 MipCount = MipChain.Num();
	check(MipCount >= (int32)ExtendedData.NumMipsInTail);
	// This number was too small (128) for current hardware and caused too many
	// context switch for work taking < 1ms. Bump the value for 2020 CPUs.
	const int32 MinAsyncCompressionSize = 512;
	const bool bAllowParallelBuild = TextureFormat->AllowParallelBuild();
	bool bCompressionSucceeded = true;

	// Mip tail is when the last few mips get grouped together in the hardware layout.
	// Treat not having a mip tail as having a mip tail with 1 mip in it, which is
	// equivalent and lets us simplify the logic.
	int32 FirstMipTailIndex = MipCount - 1;
	int32 MipTailCount = 1;

	if (ExtendedData.NumMipsInTail > 1)
	{
		MipTailCount = ExtendedData.NumMipsInTail;
		FirstMipTailIndex = MipCount - MipTailCount;
	}

	uint32 StartCycles = FPlatformTime::Cycles();

	// Set up one task for the base mip, one task for everything after. Since each mip level
	// has 4x the pixels as the one below it (8x for volumes), work for mip levels is highly
	// unbalanced and there's not much use spawning extra tasks past that: for a 2D texture,
	// the entire tail after the base mip (all remaining mips combined) has 1/3 the number of
	// pixels the base mip does.
	OutMips.SetNum(MipCount);

	FIntVector3 Mip0Dimensions = TextureDescription.GetMipDimensions(0);
	int32 Mip0NumSlicesNoDepth = TextureDescription.GetNumSlices_NoDepth();

	UE::Tasks::FCancellationToken* CancellationToken = UE::Tasks::FCancellationTokenScope::GetCurrentCancellationToken();

	auto ProcessMips =
		[&TextureFormat, &MipChain, &OutMips, &Mip0Dimensions, Mip0NumSlicesNoDepth, FirstMipTailIndex, MipTailCount, ExtData = ExtendedData.ExtData, &Settings, &DebugTexturePathName, bImageHasAlphaChannel, CancellationToken](int32 MipBegin, int32 MipEnd)
	{
		bool bSuccess = true;

		UE::Tasks::FCancellationTokenScope CancellationScope(CancellationToken);

		for (int32 MipIndex = MipBegin; MipIndex < MipEnd; ++MipIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CompressImage);

			// We always compress 1 mip at a time, unless the platform requests that the mip tail
			// gets packed in to a single mip level (NumMipsInTail).
			int32 MipsToCompress = 1;
			if (MipIndex == FirstMipTailIndex)
			{
				MipsToCompress = MipTailCount;
			}

			if (CancellationToken && CancellationToken->IsCanceled())
			{
				bSuccess = false;
			}
			else
			{
				bSuccess = bSuccess && TextureFormat->CompressImageEx(
					&MipChain[MipIndex],
					MipsToCompress,
					Settings,
					Mip0Dimensions,
					Mip0NumSlicesNoDepth,
					MipIndex,
					OutMips.Num(),
					DebugTexturePathName,
					bImageHasAlphaChannel,
					ExtData,
					OutMips[MipIndex]
				);
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CompressImage.Free);
				// CompressImageEx can free images as it works on them
				//	go ahead and always immediately free here for consistency

				MipChain[MipIndex].FreeData(true);
			}
		}

		return bSuccess;
	};

	if (bAllowParallelBuild &&
		FirstMipTailIndex > 0 &&
		FMath::Min(MipChain[0].SizeX, MipChain[0].SizeY) >= MinAsyncCompressionSize)
	{
		// Spawn async job to compress all mips below base
		auto AsyncTask = UE::Tasks::Launch(TEXT("Texture.CompressLowerMips"),
			[&ProcessMips, FirstMipTailIndex]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CompressLowerMips);
				return ProcessMips(1, FirstMipTailIndex + 1);
			},
			LowLevelTasks::ETaskPriority::BackgroundNormal
			//IsInGameThread() ? ETaskPriority::Normal : ETaskPriority::BackgroundNormal // ??
		);

		// Compress base mip on this thread, join with async compress of other mips
		bCompressionSucceeded = ProcessMips(0, 1);
		bCompressionSucceeded &= AsyncTask.GetResult();
	}
	else
	{
		// Compress all mips at once on this thread
		bCompressionSucceeded = ProcessMips(0, FirstMipTailIndex + 1);
	}

	// Fill out the dimensions for the packed mip tail, should we have one
	for (int32 MipIndex = FirstMipTailIndex + 1; MipIndex < MipCount; ++MipIndex)
	{
		FCompressedImage2D& PrevMip = OutMips[MipIndex - 1];
		FCompressedImage2D& DestMip = OutMips[MipIndex];
		DestMip.SizeX = FMath::Max(1, PrevMip.SizeX >> 1);
		DestMip.SizeY = FMath::Max(1, PrevMip.SizeY >> 1);
		DestMip.NumSlicesWithDepth = Settings.bVolume ? FMath::Max(1, PrevMip.NumSlicesWithDepth >> 1) : PrevMip.NumSlicesWithDepth;
		DestMip.PixelFormat = PrevMip.PixelFormat;
	}

	if (!bCompressionSucceeded)
	{
		OutMips.Empty();
	}

	if (CancellationToken && CancellationToken->IsCanceled())
	{
		return false;
	}

	uint32 EndCycles = FPlatformTime::Cycles();
	UE_LOGF(LogTextureCompressor,Verbose,"Compressed %dx%dx%d %ls in %fms",
		MipChain[0].SizeX,
		MipChain[0].SizeY,
		MipChain[0].NumSlices,
		*Settings.TextureFormatName.ToString(),
		FPlatformTime::ToMilliseconds( EndCycles-StartCycles )
		);

	return bCompressionSucceeded;
}

// only useful for normal maps, fixed bad input (denormalized normals) and improved quality (quantization artifacts)
static void NormalizeMip(FImage& InOutMip)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.NormalizeMip);

	//@todo Oodle : change FVector to FVector3f all over Texture code!
	const FVector NormalIfZero(0.f,0.f,1.f);

	int64 NumPixels = InOutMip.GetNumPixels();
	int64 NumPixelsEachJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForPixels(NumPixelsEachJob,NumPixels);
	
	FLinearColor * ImageColors = InOutMip.AsRGBA32F().GetData();

	ParallelFor( TEXT("Texture.NormalizeMip.PF"),NumJobs,1, [&](int32 Index)
	{
		int64 StartIndex = Index * NumPixelsEachJob;
		int64 EndIndex = FMath::Min(StartIndex + NumPixelsEachJob, NumPixels);
		for (int64 i = StartIndex; i < EndIndex; ++i)
		{
			// @todo Oodle: SIMD NormalizeColors

			FLinearColor& Color = ImageColors[i];

			// this uses doubles due to FVector :(

			FVector Normal(Color.R * 2.0f - 1.0f, Color.G * 2.0f - 1.0f, Color.B * 2.0f - 1.0f);

			// GetSafeNormal returns Vec(0,0,0) by default for tiny input, instead return flat/up
			Normal = Normal.GetSafeNormal(UE_SMALL_NUMBER,NormalIfZero);

			Color = FLinearColor(
				static_cast<float>(Normal.X * 0.5f + 0.5f), 
				static_cast<float>(Normal.Y * 0.5f + 0.5f), 
				static_cast<float>(Normal.Z * 0.5f + 0.5f), 
				Color.A);
		}
	}, EParallelForFlags::Unbalanced);
}

bool FTextureBuildSettings::GetOutputMipInfo(
	int32 InMip0SizeX, int32 InMip0SizeY, int32 InMip0NumSlices, int32 InExistingMipCount,
	int32& OutMip0SizeX, int32& OutMip0SizeY, int32& OutMip0NumSlices, int32& OutMipCount) const
{
	// NOTE: This gets called for the seperate blocks in a VT build and bVirtualStreaming is true for those
	// even though it's not a massive VT texture we're referring to.

	if (this->bCPUAccessible)
	{
		// CPU accessible texture generates a placeholder gpu texture with 1 mip in all cases.
		FImageInfo PlaceholderInfo;
		UE::TextureBuildUtilities::GetPlaceholderTextureImageInfo(&PlaceholderInfo);
		OutMip0SizeX = PlaceholderInfo.SizeX;
		OutMip0SizeY = PlaceholderInfo.SizeY;
		OutMip0NumSlices = PlaceholderInfo.NumSlices;
		OutMipCount = 1;
		return true;
	}

	int32 BaseSizeX = InMip0SizeX;
	int32 BaseSizeY = InMip0SizeY;
	int32 BaseSizeZ = this->bVolume ? InMip0NumSlices : 1; // Volume textures are the only type that mip their Z, arrays and cubes are fixed.
	
	ETexturePowerOfTwoSetting::Type PowerOfTwoModeLocal = (ETexturePowerOfTwoSetting::Type)this->PowerOfTwoMode;
	if (this->MipGenSettings != TMGS_LeaveExistingMips &&
		PowerOfTwoModeLocal != ETexturePowerOfTwoSetting::None)
	{
		int32 TargetSizeX, TargetSizeY, TargetSizeZ;
		bool NeedsAdjustment = UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(BaseSizeX, BaseSizeY, BaseSizeZ, this->bVolume, PowerOfTwoModeLocal, this->ResizeDuringBuildX, this->ResizeDuringBuildY, TargetSizeX, TargetSizeY, TargetSizeZ);
		if (NeedsAdjustment)
		{
			// In this case we are regenerating the entire mip chain.
			InExistingMipCount = 1;
			BaseSizeX = TargetSizeX;
			BaseSizeY = TargetSizeY;
			BaseSizeZ = TargetSizeZ; // volume textures already accounted for
		}
		// Otherwise we have valid pow2 so we can reuse any existing mips and regenerate
		// any missing tail mips.
	}

	// LatLong sources are clamped in ComputeLongLatCubemapExtents
	if (this->bLongLatSource == false)
	{
		// Max texture resolution strips off mips that are above the limit.
		//int64 MaxTextureResolution = this->MaxTextureResolution; ... ? why was this getting promoted to 64 bit.. probably because of the signed comparison below?
		int32 GeneratedMipCount = FImageCoreUtils::GetMipCountFromDimensions(BaseSizeX, BaseSizeY, BaseSizeZ, this->bVolume);
		int32 i = 0;
		for (; i < GeneratedMipCount; i++)
		{
			// The code in BuildTextureMips doesn't worry about fitting Z in volume textures...
			// \todo volume texture MaxTextureSize. The old code ignored size Z, so we do too. I'm not sure
			// there's ever a case where volume textures have a Z that's bigger than X/Y.
			int32 MipSizeX = FMath::Max<uint32>(1, BaseSizeX >> i);
			int32 MipSizeY = FMath::Max<uint32>(1, BaseSizeY >> i);
			int32 MipSizeZ = this->bVolume ? FMath::Max<uint32>(1, BaseSizeZ >> i) : BaseSizeZ;

			if ((uint32)MipSizeX <= this->MaxTextureResolution &&
				(uint32)MipSizeY <= this->MaxTextureResolution)
			{
				BaseSizeX = MipSizeX;
				BaseSizeY = MipSizeY;
				BaseSizeZ = MipSizeZ;
				break;
			}
		}

		if (this->Downscale > 1.0f)
		{
			int32 DownscaledSizeX = 0, DownscaledSizeY = 0;
			GetDownscaleFinalSizeAndClampedDownscale(BaseSizeX, BaseSizeY, FTextureDownscaleSettings(*this), DownscaledSizeX, DownscaledSizeY);

			if (this->bVolume)
			{
				UE_LOGF(LogTextureCompressor, Error, "Downscaling volumes not yet supported - should have been handled in GetTextureBuildSettings!");
				return false;
			}

			BaseSizeX = DownscaledSizeX;
			BaseSizeY = DownscaledSizeY;
		}

		// Volumes are the only thing where num slices changes.
		if (this->bVolume == false)
		{
			OutMip0NumSlices = InMip0NumSlices;
		}
		else
		{
			OutMip0NumSlices = BaseSizeZ;
		}
	}
	else
	{
		uint32 LongLatCubemapExtents = UE::TextureBuildUtilities::ComputeLongLatCubemapExtents(BaseSizeX, this->MaxTextureResolution);
		BaseSizeX = LongLatCubemapExtents;
		BaseSizeY = LongLatCubemapExtents;
		OutMip0NumSlices = 6 * InMip0NumSlices;
	}

	// At this point we have a base mip size that is valid.
	OutMip0SizeX = BaseSizeX;
	OutMip0SizeY = BaseSizeY;

	if (this->MipGenSettings == TMGS_NoMipmaps)
	{
		OutMipCount = 1;
		return true;
	}

	// LeaveExisting is often unintentionally used as "NoMipmaps", so if they bring in 1 mip, leave it as 1 mip.
	if (this->MipGenSettings == TMGS_LeaveExistingMips &&
		InExistingMipCount == 1)
	{
		OutMipCount = 1;
		return true;
	}

	// NumOutputMips is the number of mips that would be made if you made a full mip chain
	//  eg. 256 makes 9 mips , 300 also makes 9 mips
	OutMipCount = FImageCoreUtils::GetMipCountFromDimensions(BaseSizeX, BaseSizeY, BaseSizeZ, this->bVolume);
	return true;
}


/**
 * Texture compression module
 */
class FTextureCompressorModule : public ITextureCompressorModule
{
public:
	FTextureCompressorModule()
	{
	}
	
	// compute a hash used to identify the contents of a mip chain
	// this is used by bulkdata diff tool, stored in asset registry
	// it is for debug tools only and skipping it is optional
	// ComputeMipChainHash is synchronous; it starts tasks but waits on them before returning
	static uint64 ComputeMipChainHash(const TArray<FImage>& IntermediateMipChain)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.ComputeMipChainHash);

		TArray<UE::Tasks::TTask<FXxHash64>, TInlineAllocator<16>> MipHashTasks;
		MipHashTasks.Reserve(IntermediateMipChain.Num());

		TArray<FXxHash64> MipHashResults;
		MipHashResults.Reserve(IntermediateMipChain.Num());

		// Hash the mips before we compress them. This gets saved as part of the derived data and then added to the
		// diff tags during cook so we can catch determinism issues.
		FXxHash64Builder MipHashBuilder;
		for (const FImage& Mip : IntermediateMipChain)
		{
			const FImageInfo* Info = &Mip;
			// Can't just hash FImageInfo due to struct padding RNG making the hash non-deterministic.
			MipHashBuilder.Update(&Info->SizeX, sizeof(Info->SizeX));
			MipHashBuilder.Update(&Info->SizeY, sizeof(Info->SizeY));
			MipHashBuilder.Update(&Info->NumSlices, sizeof(Info->NumSlices));
			MipHashBuilder.Update(&Info->Format, sizeof(Info->Format));
			MipHashBuilder.Update(&Info->GammaSpace, sizeof(Info->GammaSpace));

			check( Mip.RawData.Num() != 0 );
			
			const int64 ChunkSize = 256*1024;
			auto MipMem = MakeMemoryView(Mip.RawData);

			if ( Mip.RawData.Num() >= ChunkSize )
			{
				// large mip, start async task
				MipHashTasks.Add(UE::Tasks::Launch(TEXT("ComputeMipChainHash"), [MipMem] { return FXxHash64::HashBufferChunked(MipMem, ChunkSize); }));
			}
			else
			{
				// do tiny mips here on the calling thread while the large mip tasks run
				MipHashResults.Add( FXxHash64::HashBufferChunked(MipMem, ChunkSize) );
			}
		}

		// now wait on the async tasks for the large mips :
		for (UE::Tasks::TTask<FXxHash64>& HashTask : MipHashTasks)
		{
			FXxHash64 MipHash = HashTask.GetResult(); // <- wait
			MipHashBuilder.Update(&MipHash.Hash, sizeof(MipHash.Hash));
		}
		
		// add in the hashes of the small mips :
		for (const FXxHash64 & MipHash : MipHashResults)
		{
			MipHashBuilder.Update(&MipHash.Hash, sizeof(MipHash.Hash));
		}

		return MipHashBuilder.Finalize().Hash;
	}

	// SourceMips can be freed by this call
	virtual bool BuildTexture(
		TArray<FImage>& SourceMips,
		TArray<FImage>& AssociatedNormalSourceMips,
		const FTextureBuildSettings& BuildSettings,
		FStringView DebugTexturePathName,
		TArray<FCompressedImage2D>& OutTextureMips,
		uint32& OutNumMipsInTail,
		uint32& OutExtData,
		UE::TextureBuildUtilities::FTextureBuildMetadata* OutMetadata
	)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(Texture.BuildTexture);

		const ITextureFormat* TextureFormat = nullptr;

		ITextureFormatManagerModule* TFM = GetTextureFormatManager();
		if (TFM)
		{
			TextureFormat = TFM->FindTextureFormat(BuildSettings.TextureFormatName);
		}
		if (TextureFormat == nullptr)
		{
			UE_LOGF(LogTextureCompressor, Warning,
				"Failed to find compressor for texture format '%ls'. [%.*ls]",
				*BuildSettings.TextureFormatName.ToString(),
				DebugTexturePathName.Len(),DebugTexturePathName.GetData()
			);

			return false;
		}

		TStringBuilder<32> AlphaLogs;
		const bool bDoDetailedAlphaLogging = CVarDetailedMipAlphaLogging.GetValueOnAnyThread() != 0;
		
		// @todo Oodle: option to dump the Source image here
		//		we have dump in TextureFormatOodle for the after-processing (before encoding) image
		//		get a dump spot for before-processing as well

		TArray<FImage> IntermediateMipChain;

		bool bSourceMipsExpectAlpha = false;
		if (bDoDetailedAlphaLogging)
		{
			AlphaLogs.Append(DebugTexturePathName);

			bSourceMipsExpectAlpha = FImageCore::DetectAlphaChannel(SourceMips[0]);

			AlphaLogs.Appendf(TEXT("\nSource Mips: %d / Analyzed: %d (%s)"), bSourceMipsExpectAlpha, BuildSettings.bHasTransparentAlpha, BuildSettings.bKnowAlphaTransparency ? TEXT("valid") : TEXT("INVALID"));
		}

		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return false;
		}

		// allow to leave texture in sRGB in case compressor accepts other than non-F32 input source
		// otherwise linearizing will force format to be RGBA32F
		const bool bNeedLinearize = !TextureFormat->CanAcceptNonF32Source(BuildSettings.TextureFormatName) || AssociatedNormalSourceMips.Num() != 0;
		if (!BuildTextureMips(SourceMips, BuildSettings, bNeedLinearize, bDoDetailedAlphaLogging ? &AlphaLogs : nullptr, IntermediateMipChain, DebugTexturePathName))
		{
			return false;
		}
		
		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return false;
		}

		SourceMips.Empty();

		// apply roughness adjustment depending on normal map variation
		if (AssociatedNormalSourceMips.Num())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Texture.AssociatedNormalSourceMips);

			// ECompositeTextureMode is only NormalRoughness
			//  AssociatedNormalSourceMips should be a normal map

			TArray<FImage> IntermediateAssociatedNormalSourceMipChain;

			FTextureBuildSettings DefaultSettings;

			// apply a smooth Gaussian filter to the top level of the normal map
			// the original comment says :
			//  "helps to reduce aliasing further"
			// what's happening here is the blur on the top mip will reduce the length of normals in rough areas
			//	whereas without it the top mip would always have normals of length 1.0 , hence zero roughness per Toksvig
			DefaultSettings.MipSharpening = -3.5f;
			//DefaultSettings.MipSharpening = -2.5f; // CB: I think smaller blend looks better but don't change existing data
			DefaultSettings.SharpenMipKernelSize = 6;
			DefaultSettings.bApplyKernelToTopMip = true;

			// important to make accurate computation with normal length
			//  note this normalizes the top mip *before* the gaussian blur
			DefaultSettings.bRenormalizeTopMip = true;
			DefaultSettings.bNormalizeNormals = false; // do not normalize after mip gen, we want shortening

			// use new mip filter setting from build settings
			DefaultSettings.bUseNewMipFilter = BuildSettings.bUseNewMipFilter;

			if (!BuildTextureMips(AssociatedNormalSourceMips, DefaultSettings, true, bDoDetailedAlphaLogging ? &AlphaLogs : nullptr, IntermediateAssociatedNormalSourceMipChain, DebugTexturePathName))
			{
				if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
				{
					return false;
				}

				UE_LOGF(LogTextureCompressor, Warning, "Failed to generate texture mips for composite texture [%.*ls]",
					DebugTexturePathName.Len(),DebugTexturePathName.GetData());

				return false;
			}

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return false;
			}

			AssociatedNormalSourceMips.Empty();

			if (!ApplyCompositeTextureToMips(IntermediateMipChain, IntermediateAssociatedNormalSourceMipChain, BuildSettings.CompositeTextureMode, BuildSettings.CompositePower, BuildSettings.LODBias))
			{
				if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
				{
					return false;
				}

				UE_LOGF(LogTextureCompressor, Display, "ApplyCompositeTextureToMips failed [%.*ls]",
					DebugTexturePathName.Len(),DebugTexturePathName.GetData());

				return false;
			}

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return false;
			}

			if (bDoDetailedAlphaLogging)
			{
				AlphaLogs.Appendf(TEXT("\nAssociated Normal Mips: %d"), FImageCore::DetectAlphaChannel(IntermediateMipChain[0]));
			}
		}

		bool bKnowAlphaTransparency = BuildSettings.bKnowAlphaTransparency;
		bool bHasTransparentAlpha = BuildSettings.bHasTransparentAlpha;
		
		// We expect virtually all textures coming through this path to carry color information as it's scanned when the
		// source mips are retrieved for the build. (FTextureSourceData::GetSourceMips). However if the settings don't allow
		// for prediction off of source mips, we evaluate here.
		if (!bKnowAlphaTransparency)
		{
			// We only need the alpha information if our pixel format can change as a result. This should only be
			// the various "Auto" formats, but the safest way to check is to try both and see. Currenly the only known
			// way to get here results in formats that do NOT change based on the presence of alpha.
			EPixelFormat FormatWithAlpha = TextureFormat->GetEncodedPixelFormat(BuildSettings, true);
			EPixelFormat FormatWithoutAlpha = TextureFormat->GetEncodedPixelFormat(BuildSettings, false);
			if (FormatWithAlpha != FormatWithoutAlpha)
			{
				// At this time there's no known way to get here. Setting as a warning so hopefully we hear
				// about it when it happens.
				UE_LOGF(LogTextureCompressor, Warning, "Scanning intermediate mips for alpha: %.*ls", DebugTexturePathName.Len(), DebugTexturePathName.GetData());

				bool bIntermediateMipsAlphaDetected = FImageCore::DetectAlphaChannel(IntermediateMipChain[0]);
				bHasTransparentAlpha = BuildSettings.GetTextureExpectsAlphaInPixelFormat(bIntermediateMipsAlphaDetected);
				bKnowAlphaTransparency = true;
			}
			else
			{
				// If we don't think we need it, make sure that we use false for alpha in case something happens
				// to look at it we have a consistent answer.
				bHasTransparentAlpha = false;
			}
		}

		if (bDoDetailedAlphaLogging)
		{
			bool bIntermediateMipsAlphaDetected = FImageCore::DetectAlphaChannel(IntermediateMipChain[0]);
			AlphaLogs.Appendf(TEXT("\nIntermediate Mips: %d"), bIntermediateMipsAlphaDetected);
			AlphaLogs.Appendf(TEXT("\nAlpha Channel Needed: %d"), 
				   BuildSettings.GetTextureExpectsAlphaInPixelFormat(bIntermediateMipsAlphaDetected));

			EPixelFormat FormatWithAlpha = TextureFormat->GetEncodedPixelFormat(BuildSettings, true);
			EPixelFormat FormatWithoutAlpha = TextureFormat->GetEncodedPixelFormat(BuildSettings, false);

			if (FormatWithAlpha != FormatWithoutAlpha &&
				BuildSettings.GetTextureExpectsAlphaInPixelFormat(bIntermediateMipsAlphaDetected) != bSourceMipsExpectAlpha)
			{
				AlphaLogs.Appendf(TEXT("\nSource / Intermediate diff (force %d force no %d)"),
					BuildSettings.bForceAlphaChannel, 
					BuildSettings.bForceNoAlphaChannel);
				
				UE_LOGF(LogTextureCompressor, Warning, "%ls", *AlphaLogs);
			}
			else
			{
				UE_LOGF(LogTextureCompressor, Display, "%ls", *AlphaLogs);
			}
		}
		
		if (OutMetadata)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Texture.MipHashBuilder);

			OutMetadata->PreEncodeMipsHash = ComputeMipChainHash(IntermediateMipChain);
		}
		
		// CompressMipChain may free IntermediateMipChain
		bool bCompressSucceeded = CompressMipChain(TextureFormat, IntermediateMipChain, BuildSettings, bHasTransparentAlpha, DebugTexturePathName,
					OutTextureMips, OutNumMipsInTail, OutExtData);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Texture.Free_IntermediateMipChain);

			// this is no longer slow because mips are freed as they're consumed in CompressMipChain
			IntermediateMipChain.Empty();
		}

		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return false;
		}

		if ( bCompressSucceeded )
		{
			// make sure this log is done before the decode for platform preview
			//  -logcmds="LogTextureUpload Verbose,LogTextureCompressor Verbose"

			for(int32 MipIndex=0;MipIndex < OutTextureMips.Num();MipIndex++)
			{
				const FCompressedImage2D & CompressedImage = OutTextureMips[MipIndex];

				int64 CompressedDataSize = CompressedImage.RawData.Num();

				// log what the TextureFormat built :
				UE_LOGF(LogTextureCompressor, Verbose, "Built texture: [%.*ls] Compressed Mip %d PF=%d=%ls : %dx%dx%d : CompressedDataSize=%lld",
					DebugTexturePathName.Len(),DebugTexturePathName.GetData(),
					MipIndex, (int)CompressedImage.PixelFormat,
					GetPixelFormatString((EPixelFormat)CompressedImage.PixelFormat),
					CompressedImage.SizeX, CompressedImage.SizeY, CompressedImage.NumSlicesWithDepth,
					CompressedDataSize);
			}
		}

		return bCompressSucceeded;
	}

	// IModuleInterface implementation.
	void StartupModule() override
	{
		// Ensure the OpenColorIO wrapper module is loaded first.
		IOpenColorIOWrapperModule::Get();
	}

	void ShutdownModule() override
	{
	}

private:


	// InSourceMipChain can be freed by this call
	bool BuildTextureMips(
		TArray<FImage>& InSourceMipChain,
		const FTextureBuildSettings& BuildSettings,
		const bool bNeedLinearize,
		TStringBuilder<32>* InAlphaLogs,
		TArray<FImage>& OutMipChain,
		FStringView DebugTexturePathName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.BuildTextureMips);
		
		check(InSourceMipChain.Num() > 0);
		check(InSourceMipChain[0].SizeX > 0 && InSourceMipChain[0].SizeY > 0 && InSourceMipChain[0].NumSlices > 0);
		
		int32 InSourceMipChainNum = InSourceMipChain.Num();

		// Calculation check before we change anything, because source mips can be freed :
		int32 CalculatedMip0SizeX, CalculatedMip0SizeY, CalculatedMip0NumSlices;
		int32 CalculatedMipCount = GetMipCountForBuildSettings(
			InSourceMipChain[0].SizeX, InSourceMipChain[0].SizeY, InSourceMipChain[0].NumSlices, InSourceMipChain.Num(), BuildSettings,
			CalculatedMip0SizeX, CalculatedMip0SizeY, CalculatedMip0NumSlices);

		// Identify long-lat cubemaps.
		const bool bLongLatCubemap = BuildSettings.bLongLatSource;
		if (BuildSettings.bCubemap && !bLongLatCubemap)
		{
			if (BuildSettings.bTextureArray && (InSourceMipChain[0].NumSlices % 6) != 0)
			{
				// Cube array must have multiple of 6 slices
				return false;
			}
			if (!BuildSettings.bTextureArray && InSourceMipChain[0].NumSlices != 6)
			{
				// Non-array cube must have exactly 6 slices
				return false;
			}
		}

		// handling of bLongLatCubemap seems overly complicated
		//	what it should do is convert it right at the start here
		//	then treat it as a standard cubemap below, no special cases
		//	but that will change output :(

		// pSourceMips will track the current FImages we consider to be "source"
		//	whenever pSourceMips is changed, previous pSourceMips is freed
		TArray<FImage> * pSourceMips = &InSourceMipChain;

		// first pad up to pow2 if requested
		ETexturePowerOfTwoSetting::Type PowerOfTwoMode = (ETexturePowerOfTwoSetting::Type) BuildSettings.PowerOfTwoMode;

		// if bPadOrStretchTextureis done, PaddedSourceMips is filled with a new image
		TArray<FImage> PaddedSourceMips;

		if ( PowerOfTwoMode != ETexturePowerOfTwoSetting::None )
		{
			const FImage& FirstSourceMipImage = (*pSourceMips)[0];

			int32 TargetTextureSizeX = 0;
			int32 TargetTextureSizeY = 0;
			int32 TargetTextureSizeZ = 0;			
			bool bPadOrStretchTexture = UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(
				FirstSourceMipImage.SizeX, FirstSourceMipImage.SizeY, FirstSourceMipImage.NumSlices,
				BuildSettings.bVolume, PowerOfTwoMode, BuildSettings.ResizeDuringBuildX, BuildSettings.ResizeDuringBuildY,
				TargetTextureSizeX, TargetTextureSizeY, TargetTextureSizeZ);

			if (bPadOrStretchTexture)
			{
				if (BuildSettings.MipGenSettings == TMGS_LeaveExistingMips)
				{
					// pad/stretch+leave existing is broken
					UE_LOGF(LogTextureCompressor, Error,	"Texture padding or resizing is not allowed when leaving existing mips.");
					return false;
				}

				// Want to stretch or pad the texture
				bool bResizeTexture = PowerOfTwoMode == ETexturePowerOfTwoSetting::StretchToPowerOfTwo || PowerOfTwoMode == ETexturePowerOfTwoSetting::StretchToSquarePowerOfTwo || PowerOfTwoMode == ETexturePowerOfTwoSetting::ResizeToSpecificResolution;
				// if not bResizeTexture, then padding

				bool bSuitableFormat = FirstSourceMipImage.Format == ERawImageFormat::RGBA32F;

				FImage Temp;
				if (!bSuitableFormat)
				{
					// convert to RGBA32F
					FirstSourceMipImage.CopyTo(Temp, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
				}

				// space for one source mip and one destination mip
				const FImage& SourceImage = bSuitableFormat ? FirstSourceMipImage : Temp;
				FImage& TargetImage = PaddedSourceMips.Emplace_GetRef(TargetTextureSizeX, TargetTextureSizeY, BuildSettings.bVolume ? TargetTextureSizeZ : SourceImage.NumSlices, SourceImage.Format);

				if (bResizeTexture)
				{		
					// changed resizer for "stretch to power of two" , bumped ddc key

					// choice of Filter? wrap or clamp?

					// if you have multiple resize operations on a texture, such as StrechToPow2 and then also MaxTextureSize or Downscale
					//	currently those are done one by one, which is not great
					//	would be better to compute the net output size of all the operations
					//	and do just one resize to get directly from source to final size

					FImageCore::ResizeImageAllocDest(SourceImage,TargetImage, TargetTextureSizeX, TargetTextureSizeY);
				}
				else
				{
					FLinearColor FillColor = BuildSettings.PaddingColor;
					bool bPadWithBorderColor = BuildSettings.bPadWithBorderColor;

					FLinearColor* TargetPtr = (FLinearColor*)TargetImage.RawData.GetData();
					FLinearColor* SourcePtr = (FLinearColor*)SourceImage.RawData.GetData();
					check(SourceImage.GetBytesPerPixel() == sizeof(FLinearColor));
					check(TargetImage.GetBytesPerPixel() == sizeof(FLinearColor));

					for (int32 SliceIndex = 0; SliceIndex < SourceImage.NumSlices; ++SliceIndex)
					{
						for (int32 Y = 0; Y < TargetTextureSizeY; ++Y)
						{
							if (Y < SourceImage.SizeY)
							{
								FMemory::Memcpy(TargetPtr, SourcePtr, SourceImage.SizeX * sizeof(FLinearColor));
								SourcePtr += SourceImage.SizeX;
								TargetPtr += SourceImage.SizeX;
								
								int32 PadCount = TargetImage.SizeX - SourceImage.SizeX;
								if ( bPadWithBorderColor )
								{
									const FLinearColor BorderColor = TargetPtr[-1];
									for (int32 i=0;i<PadCount;i++)
									{
										*TargetPtr++ = BorderColor;
									}
								}
								else
								{
									for (int32 i=0;i<PadCount;i++)
									{
										*TargetPtr++ = FillColor;
									}
								}
							}
							else if (bPadWithBorderColor)
							{
								// We're copying the entirely of the last line of the target image, which we know has proper padding horizontally
								// because earlier passes (when Y < SourceImage.SizeY) will pad out the line in the loop below.
								FMemory::Memcpy(TargetPtr, TargetPtr - TargetImage.SizeX, TargetImage.SizeX * sizeof(FLinearColor));
								TargetPtr += TargetImage.SizeX;
							}
							else
							{
								// fill whole line with FillColor
								
								for (int32 i=0;i<TargetImage.SizeX;i++)
								{
									*TargetPtr++ = FillColor;
								}
							}
						}
					}
					// Pad new slices for volume texture
					for (int32 SliceIndex = SourceImage.NumSlices; SliceIndex < TargetImage.NumSlices; ++SliceIndex)
					{
						for (int32 Y = 0; Y < TargetImage.SizeY; ++Y)
						{
							if (bPadWithBorderColor)
							{
								// We're copying the entirely of the corresponding line from the previous slice, which we know has proper padding
								// performed during earlier passes (SliceIndex < SourceImage.NumSlices).
								FMemory::Memcpy(TargetPtr, TargetPtr - (int64) TargetImage.SizeX * TargetImage.SizeY, TargetImage.SizeX * sizeof(FLinearColor));
								TargetPtr += TargetImage.SizeX;
							}
							else
							{
								for (int32 X = 0; X < TargetImage.SizeX; ++X)
								{
									*TargetPtr++ = FillColor;
								}
							}
						}
					}
				}
				// change pSourceMips to point at the one padded image we made
				pSourceMips->Empty();
				pSourceMips = &PaddedSourceMips;
			}
		}		

		// now pow2 pad is done
		// find a starting source that meets MaxTextureResolution limit

		int32 StartMip = 0;

		TArray<FImage> BuildSourceImageMips;

		if ( ! bLongLatCubemap )
		{
			int32 NumSourceMips = (BuildSettings.MipGenSettings == TMGS_LeaveExistingMips) ? pSourceMips->Num() : 1;
			
			int64 MaxTextureResolution = BuildSettings.MaxTextureResolution;

			// note that "LODBias" is very similar to MaxTextureResolution
			//	but for LODBias we go ahead and make all the mips here
			//	and then just don't serialize the top ones in TextureDerivedData
			// (LODBias is not actually LOD Bias, it means discard top N mips)

			// step through source mips to find one that meets MaxTextureResolution
			while( StartMip < NumSourceMips && (
				(*pSourceMips)[StartMip].SizeX > MaxTextureResolution ||
				(*pSourceMips)[StartMip].SizeY > MaxTextureResolution ) )
			{
				StartMip++;
			}

			if ( StartMip == NumSourceMips )
			{

				// bLongLatCubemap should not get here because cube size is made from MaxTextureSize
				check( ! bLongLatCubemap );

				// the source is larger than the compressor allows and no mip image exists to act as a smaller source.
				// We must generate a suitable source image:
				const FImage& BaseImage = pSourceMips->Last();
			
				check( MaxTextureResolution > 0 );
				check( BaseImage.SizeX > MaxTextureResolution || 
					   BaseImage.SizeY > MaxTextureResolution );
					   
				UE_LOGF(LogTextureCompressor, Verbose,
					"Source image %dx%d too large for compressors max dimension (%d). Resizing.",
					BaseImage.SizeX,
					BaseImage.SizeY,
					BuildSettings.MaxTextureResolution
					);

				if ( BuildSettings.bUseNewMipFilter && 
					! BuildSettings.bVolume && ! BuildSettings.bDoScaleMipsForAlphaCoverage && ! BuildSettings.bPreserveBorder )
				{
					// BuildSourceImageMips will be used as the source for further steps, fill it :
					BuildSourceImageMips.SetNum(1);
					FImage & DestImage = BuildSourceImageMips[0];

					// note we could resize directly to MaxTextureResolution
					//	for now we will replicate the old logic of only doing 2X mip size steps
					//	this results in output that is in (MaxTextureResolution/2,MaxTextureResolution]
					int64 DestSizeX = BaseImage.SizeX;
					int64 DestSizeY = BaseImage.SizeY;
					while( DestSizeX > MaxTextureResolution || DestSizeY > MaxTextureResolution )
					{
						DestSizeX = FMath::Max(DestSizeX>>1,1);
						DestSizeY = FMath::Max(DestSizeY>>1,1);
					}

					DestImage.Init(DestSizeX,DestSizeY,BaseImage.NumSlices,ERawImageFormat::RGBA32F, EGammaSpace::Linear);

					// rather than doing a bunch of mip-step resizes,
					//	just resize once directly to dest size using ResizeImage
					//	also the BaseImage stays in BGRA8, no need to promote it to RGBA32F

					// ResizeImage does work with Slices for cubes/arrays , but not volumes
					check( ! BuildSettings.bVolume );

					// @todo : find closest ResizeFilter that matches MipGen settings (maybe?)
					//	 I mean, maybe not?  Most of the time MipGen is "SimpleAverage" which produces a really bad resize
					//	-> wrap/clamp ?
					FImageCore::EResizeImageFilter ResizeFilter = FImageCore::EResizeImageFilter::Default;
					FImageCore::ResizeImage(BaseImage,DestImage,ResizeFilter);
				}
				else
				{
					// old way

					FImage Temp;
					bool bSuitableFormat = BaseImage.Format == ERawImageFormat::RGBA32F;
					if (!bSuitableFormat)
					{
						// convert to RGBA32F
						BaseImage.CopyTo(Temp, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
					}

					// make sure BuildSourceImageMips doesn't reallocate :
					constexpr int BuildSourceImageMipsMaxCount = 20; // plenty
					BuildSourceImageMips.Empty(BuildSourceImageMipsMaxCount);

					// Max Texture Size resizing happens here :
					// note we do not check for TMGS_Angular here
					// note that TMGS_NoMipMaps *can* use this path; in that case it generates mips using 2x2 simple average
					const FImage& BaseMip = bSuitableFormat ? BaseImage : Temp;
					GenerateMipChain(BuildSettings, BaseMip, BuildSourceImageMips, 1);

					// mip data not needed anymore
					Temp.RawData.Empty();					
					pSourceMips->Empty();

					// note: on volumes, only XY size is checked, but Z size changes too
					while( BuildSourceImageMips.Last().SizeX > MaxTextureResolution || 
						   BuildSourceImageMips.Last().SizeY > MaxTextureResolution )
					{
						// note: now making mips one by one, rather than N in one call
						//	this is not exactly the same if AlphaCoverage processing is on
						check( BuildSourceImageMips.Num() < BuildSourceImageMipsMaxCount );
						FImage& LastMip = BuildSourceImageMips.Last();
						GenerateMipChain(BuildSettings, LastMip, BuildSourceImageMips, 1);

						// mip data not needed anymore
						LastMip.RawData.Empty();
					}
				}				

				check( BuildSourceImageMips.Last().SizeX <= MaxTextureResolution &&
						BuildSourceImageMips.Last().SizeY <= MaxTextureResolution );

				// change pSourceMips to point at the mip chain we made				
				pSourceMips->Empty();
				pSourceMips = &BuildSourceImageMips;
				StartMip = BuildSourceImageMips.Num() - 1;

				// [StartMip] will now references BuildSourceImageMips.Last()
			}
		}

		// now shrinking to MaxTextureResolution is done, figure out which mips to use or make
		
		// Copy over base mip and any LeaveExisting, from SourceMips , starting at StartMip
		int32 CopyCount = pSourceMips->Num() - StartMip;
		check( CopyCount > 0 );
		
		int32 NumOutputMips;

		if ( BuildSettings.MipGenSettings == TMGS_NoMipmaps )
		{
			NumOutputMips = 1;
			CopyCount = 1;
		}
		else
		{
			const FImage & TopMip = (*pSourceMips)[StartMip];

			// NumOutputMips is the number of mips that would be made if you made a full mip chain
			//  eg. 256 makes 9 mips , 300 also makes 9 mips
			if (bLongLatCubemap)
			{
				uint32 Extents = UE::TextureBuildUtilities::ComputeLongLatCubemapExtents(TopMip.SizeX, BuildSettings.MaxTextureResolution);
				NumOutputMips = FImageCoreUtils::GetMipCountFromDimensions(Extents, Extents, 6, false);
			}
			else
			{
				NumOutputMips = FImageCoreUtils::GetMipCountFromDimensions(TopMip.SizeX, TopMip.SizeY, TopMip.NumSlices, BuildSettings.bVolume);
			}

			// LeaveExistingMips is often inadvertently used as "NoMips", so if they only brought in 1 mip, leave it as
			// 1 mip.
			if (BuildSettings.MipGenSettings == TMGS_LeaveExistingMips &&
				InSourceMipChainNum == 1)
			{
				NumOutputMips = 1;
				check(CopyCount == 1);
			}

			// unless LeaveExistingMips, we only copy 1
			if (BuildSettings.MipGenSettings != TMGS_LeaveExistingMips)
			{
				CopyCount = 1;
			}
		}

		// we will output NumOutputMips
		OutMipChain.Empty(NumOutputMips);

		int32 GenerateCount = NumOutputMips - CopyCount;
		check( GenerateCount >= 0 );

		// avoid converting to RGBA32F linear format if there's no need for any extra processing of pixels
		// image will be left in BGRA8 format if possible
		const bool bNeedAdjustImageColors = NeedAdjustImageColors(BuildSettings);
		const bool bLinearize = bNeedLinearize || (GenerateCount > 0) || BuildSettings.bRenormalizeTopMip || (BuildSettings.Downscale > 1.f)
			|| BuildSettings.bHasColorSpaceDefinition || BuildSettings.bComputeBokehAlpha || BuildSettings.bFlipGreenChannel
			|| BuildSettings.bReplicateRed || BuildSettings.bReplicateAlpha || BuildSettings.bApplyYCoCgBlockScale
			|| (BuildSettings.SourceEncodingOverride != 0) || bNeedAdjustImageColors || BuildSettings.bNormalizeNormals;

		for (int32 MipIndex = StartMip; MipIndex < StartMip + CopyCount; ++MipIndex)
		{
			FImage& Image = (*pSourceMips)[MipIndex];

			if (InAlphaLogs)
			{
				InAlphaLogs->Appendf(TEXT("\nPre Process: %d"), FImageCore::DetectAlphaChannel(Image));
			}
			
			// copy mips over + processing
			// this is a code dupe of the processing done in GenerateMipChain

			// create base for the mip chain
			FImage& Mip = OutMipChain.AddDefaulted_GetRef();

			if (bLongLatCubemap)
			{
				// Generate the base mip from the long-lat source image.
				GenerateBaseCubeMipFromLongitudeLatitude2D(&Mip, Image, BuildSettings);
				check( CopyCount == 1 );
			}
			else
			{
				// copy base source content to the base of the mip chain
				if(BuildSettings.bApplyKernelToTopMip)
				{
					FImage Temp;
					LinearizeToWorkingColorSpace(Image, Temp, BuildSettings);
					if(BuildSettings.bRenormalizeTopMip)
					{
						NormalizeMip(Temp);
					}

					GenerateTopMip(Temp, Mip, BuildSettings);

					if (InAlphaLogs)
					{
						InAlphaLogs->Appendf(TEXT("\nApplyKernel: %d"), FImageCore::DetectAlphaChannel(Mip));
					}
				}
				else
				{
					if (bLinearize)
					{
						LinearizeToWorkingColorSpace(Image, Mip, BuildSettings);
						if (BuildSettings.bRenormalizeTopMip)
						{
							NormalizeMip(Mip);
						}

						if (InAlphaLogs)
						{
							InAlphaLogs->Appendf(TEXT("\nLinearize: %d"), FImageCore::DetectAlphaChannel(Mip));
						}
					}
					else
					{
						// just move Image to Mip, no copy
						Image.Swap(Mip);

						if (InAlphaLogs)
						{
							InAlphaLogs->Appendf(TEXT("\nCopy: %d"), FImageCore::DetectAlphaChannel(Mip));
						}
					}
				}
			}

			// free pSourceMips as we consume them (detached)
			Image.FreeData(true);

			if (BuildSettings.Downscale > 1.f)
			{		
				DownscaleImage(Mip, Mip, FTextureDownscaleSettings(BuildSettings));

				if (InAlphaLogs)
				{
					InAlphaLogs->Appendf(TEXT("\nDownscale: %d"), FImageCore::DetectAlphaChannel(Mip));
				}
			}

			// Apply color adjustments
			AdjustImageColors(Mip, BuildSettings);

			if (InAlphaLogs)
			{
				InAlphaLogs->Appendf(TEXT("\nColorAdjust: %d"), FImageCore::DetectAlphaChannel(Mip));
			}

			if (BuildSettings.bComputeBokehAlpha)
			{
				// To get the occlusion in the BokehDOF shader working for all Bokeh textures.
				ComputeBokehAlpha(Mip);

				if (InAlphaLogs)
				{
					InAlphaLogs->Appendf(TEXT("\nBokeh: %d"), FImageCore::DetectAlphaChannel(Mip));
				}
			}
			if (BuildSettings.bFlipGreenChannel)
			{
				FlipGreenChannel(Mip);
			}
		}

		check( OutMipChain.Num() == CopyCount );
		check( GenerateCount == NumOutputMips - OutMipChain.Num() );

		// no further reads from source:
		pSourceMips->Empty();
		pSourceMips = nullptr;

		// Generate any missing mips in the chain.
		if ( GenerateCount > 0 )
		{
			// Do angular filtering of cubemaps if requested.
			if (BuildSettings.MipGenSettings == TMGS_Angular)
			{
				check( BuildSettings.bCubemap );
				// note TMGS_Angular forces dim to next lower power of 2

				// note GenerateAngularFilteredMips reprocesses ALL the mips, not just GenerateCount
				// this should probably be outside the GenerateCount check (eg. always done, even if GenerateCount == 0 )
				// but putting it inside matches existing behavior
				// I guess it's moot because you can't set NoMipMips or LeaveExisting if you chose Angular

				GenerateAngularFilteredMips(OutMipChain, NumOutputMips, BuildSettings.DiffuseConvolveMipLevel);
			}
			else
			{
				// GenerateMipChain should bring us up to NumOutputMips
				//	but it doesn't take NumOutputMips as a param, makes its own decision
				//  we will check that it chose the same mip count after

				// you could pass GenerateCount as the large arg here
				//  and it should make the same result

				int StartImageIndex = OutMipChain.Num()-1;
				GenerateMipChain(BuildSettings, OutMipChain[StartImageIndex], OutMipChain, MAX_uint32);
			}
		}
		check(OutMipChain.Num() == NumOutputMips);

		if (CalculatedMipCount != NumOutputMips ||
			CalculatedMip0SizeX != OutMipChain[0].SizeX ||
			CalculatedMip0SizeY != OutMipChain[0].SizeY ||
			CalculatedMip0NumSlices != OutMipChain[0].NumSlices)
		{
			UE_LOGF(LogTextureCompressor, Error, "Texture %.*ls generated unexpected mip chain: GetMipCountForBuildSettings expected %d mips, %dx%dx%d, got %d mips, %dx%dx%d!", 
				DebugTexturePathName.Len(), DebugTexturePathName.GetData(), 
				CalculatedMipCount, CalculatedMip0SizeX, CalculatedMip0SizeY, CalculatedMip0NumSlices,
				NumOutputMips, OutMipChain[0].SizeX, OutMipChain[0].SizeY, OutMipChain[0].NumSlices);
		}

		if (InAlphaLogs)
		{
			InAlphaLogs->Appendf(TEXT("\nBeginPost: %d"), FImageCore::DetectAlphaChannel(OutMipChain[0]));
		}


		// Apply post-mip generation adjustments.
		if ( BuildSettings.bNormalizeNormals )
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Texture.NormalizeMipChain);
			
			// Parallel on the mips, but not on tiny mips :
			int32 NumParallelMips = OutMipChain.Num();
			while ( NumParallelMips > 0 && OutMipChain[NumParallelMips-1].GetNumPixels() < 16384 )
			{
				NumParallelMips--;
			}

			ParallelFor( TEXT("Texture.NormalizeMipChain.PF"),NumParallelMips,1, [&](int32 Mip)
			{
				if ( Mip == NumParallelMips-1 )
				{
					// tiny mips in the tail done on one job:
					do
					{
						NormalizeMip(OutMipChain[Mip]);
						++Mip;
					} while( Mip < OutMipChain.Num() );
				}
				else
				{
					NormalizeMip(OutMipChain[Mip]);
				}
			}, EParallelForFlags::Unbalanced);

		}
	
		if (BuildSettings.bReplicateRed)
		{
			check( !BuildSettings.bReplicateAlpha ); // cannot both be set 
			ReplicateRedChannel(OutMipChain);

			if (InAlphaLogs)
			{
				InAlphaLogs->Appendf(TEXT("\nReplicateRed: %d"), FImageCore::DetectAlphaChannel(OutMipChain[0]));
			}
		}
		else if (BuildSettings.bReplicateAlpha)
		{
			ReplicateAlphaChannel(OutMipChain);
		}

		if (BuildSettings.bApplyYCoCgBlockScale)
		{
			ApplyYCoCgBlockScale(OutMipChain);
		}

		if (InAlphaLogs)
		{
			InAlphaLogs->Appendf(TEXT("\nEnd: %d"), FImageCore::DetectAlphaChannel(OutMipChain[0]));
		}

		return true;
	}

	// @param CompositeTextureMode original type ECompositeTextureMode
	// @return true on success, false on failure. Can fail due to bad mismatched dimensions of incomplete mip chains.
	bool ApplyCompositeTextureToMips(TArray<FImage>& DestRoughnessMips, const TArray<FImage>& NormalSourceMips, uint8 CompositeTextureMode, float CompositePower, int32 DestLODBias)
	{
		check( DestRoughnessMips.Num() > 0 );
		check( NormalSourceMips.Num() > 0 );

		// NormalSourceMips is always a full mip chain, because we just made it, ignoring MipGen settings on the normal map texture
		// DestRoughnessMips are the mips being made in the current texture build, they could be an incomplete set if NoMipMips or LeaveExisting

		// must write to every mip of output :
		for(int32 DestLevel = 0; DestLevel < DestRoughnessMips.Num(); ++DestLevel)
		{
			if ( DestRoughnessMips[DestLevel].SizeX > NormalSourceMips[0].SizeX &&
				 DestRoughnessMips[DestLevel].SizeY > NormalSourceMips[0].SizeY )
			{
				// Normal map size is smaller than dest, no Roughness needed
				// at equal size, DO compute roughness as a Gaussian has been applied to filter normals
				//  to find a roughness at the top mip level
				continue;
			}

			// find a mip of source normals that is the same size as current mip, if possible
			int32 SourceNormalMipLevel = FMath::Min(DestLevel,NormalSourceMips.Num()-1);
			while( SourceNormalMipLevel > 0 && (
				NormalSourceMips[SourceNormalMipLevel].SizeX < DestRoughnessMips[DestLevel].SizeX ||
				NormalSourceMips[SourceNormalMipLevel].SizeY < DestRoughnessMips[DestLevel].SizeY) )
			{
				SourceNormalMipLevel--;
			}
			while( SourceNormalMipLevel < NormalSourceMips.Num()-1 && (
				NormalSourceMips[SourceNormalMipLevel].SizeX > DestRoughnessMips[DestLevel].SizeX ||
				NormalSourceMips[SourceNormalMipLevel].SizeY > DestRoughnessMips[DestLevel].SizeY) )
			{
				SourceNormalMipLevel++;
			}

			if ( 
				DestRoughnessMips[DestLevel].SizeX != NormalSourceMips[SourceNormalMipLevel].SizeX ||
				DestRoughnessMips[DestLevel].SizeY != NormalSourceMips[SourceNormalMipLevel].SizeY )
			{
				UE_LOGF(LogTextureCompressor, Display, 
					"ApplyCompositeTexture: Couldn't find matching mip size, will stretch.  (dest: %dx%d with %d mips, source: %dx%d with %d mips).  current: (dest: %dx%d at %d, source: %dx%d at %d)",
					DestRoughnessMips[0].SizeX,
					DestRoughnessMips[0].SizeY,
					DestRoughnessMips.Num(),
					NormalSourceMips[0].SizeX,
					NormalSourceMips[0].SizeY,
					NormalSourceMips.Num(),
					DestRoughnessMips[DestLevel].SizeX,
					DestRoughnessMips[DestLevel].SizeY,
					DestLevel,
					NormalSourceMips[SourceNormalMipLevel].SizeX,
					NormalSourceMips[SourceNormalMipLevel].SizeY,
					SourceNormalMipLevel
					);
			}

			if ( ! ApplyCompositeTexture(DestRoughnessMips[DestLevel], NormalSourceMips[SourceNormalMipLevel], CompositeTextureMode, CompositePower) )
			{
				return false;
			}
		}

		return true;
	}
};

IMPLEMENT_MODULE(FTextureCompressorModule, TextureCompressor)
