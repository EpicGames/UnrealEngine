// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Serialization/CompactBinary.h"

#define TEXTURE_COMPRESSOR_MODULENAME "TextureCompressor"

struct FEncodedTextureDescription;
class ITextureFormat;
class ITextureTiler;
struct FTextureEngineParameters;
enum EPixelFormat : uint8;
namespace UE::TextureBuildUtilities { struct FTextureBuildMetadata; };

/**
 * Compressed image data.
 */
struct FCompressedImage2D
{
	TArray64<uint8> RawData;
	// in the past Sizes here were aligned up to a compressed block size multiple
	//	that is no longer done, the real size is stored
	int32 SizeX;
	int32 SizeY;

	UE_DEPRECATED(5.5, "Use NumSlicesWithDepth or GetRHIStyleSizeZ instead")
	int32 SizeZ;

	// See FEncodedTextureDescription::GetNumSlices_WithDepth.
	// Cubemaps = 6, Arrays = Count, Cubemaparrays = Count*6, Volume = Depth
	int32 NumSlicesWithDepth;

	
	// This is the SizeZ that gets passed to the RHI texture mip map stuff. See FStreamableTextureResource::SizeZ.
	// It's weird because it's actually NOT used for cubemaps or cubemap arrays, however it does get saved in derived data
	// in Texture2DMipMap, so we need to continue to pass through the previous values for cubemaps which are:
	//		*non array* cubemap = 1
	//		*array* cubemaps = array_count * 6
	//		volume = depth
	//		arrays = array_count
	int32 GetRHIStyleSizeZ(bool bTextureArray, bool bVolume) const 
	{
		return (bVolume || bTextureArray) ? NumSlicesWithDepth : 1;
	}

	EPixelFormat PixelFormat;
};

/**
 * Color adjustment parameters.
 */
struct FColorAdjustmentParameters 
{
	/** Brightness adjustment (scales HSV value) */
	float AdjustBrightness;

	/** Curve adjustment (raises HSV value to the specified power) */
	float AdjustBrightnessCurve;

	/** Saturation adjustment (scales HSV saturation) */
	float AdjustSaturation;

	/** "Vibrance" adjustment (HSV saturation algorithm adjustment) */
	float AdjustVibrance;

	/** RGB curve adjustment (raises linear-space RGB color to the specified power) */
	float AdjustRGBCurve;

	/** Hue adjustment (offsets HSV hue by value in degrees) */
	float AdjustHue;

	/** Remaps the alpha to the specified min/max range  (Non-destructive; Requires texture source art to be available.) */
	float AdjustMinAlpha;

	/** Remaps the alpha to the specified min/max range  (Non-destructive; Requires texture source art to be available.) */
	float AdjustMaxAlpha;

	/** Constructor */
	FColorAdjustmentParameters()
		: AdjustBrightness( 1.0f ),
		  AdjustBrightnessCurve( 1.0f ),
		  AdjustSaturation( 1.0f ),
		  AdjustVibrance( 0.0f ),
		  AdjustRGBCurve( 1.0f ),
		  AdjustHue( 0.0f ),
		  AdjustMinAlpha( 0.0f ),
		  AdjustMaxAlpha( 1.0f )
	{
	}
};

/**
 * Texture build settings.
 */
struct FTextureBuildSettings
{
	/** Format specific config object view or null if no format specific config is applied as part of this build. This is only for DDC2 builds,
	* and gets created when the build settings gets serialized for sending - it is not valid beforehand. Meaning, it should only ever be read from
	* build workers. It is used in place of INIs or command line args to configure individual texture formats since those are not available to
	* the texture build workers.
	*/
	FCbObjectView FormatConfigOverride;
	/** Color adjustment parameters. */
	FColorAdjustmentParameters ColorAdjustment;
	/** Enable preserving alpha coverage. */
	bool bDoScaleMipsForAlphaCoverage;
	/** Channel values to compare to when preserving alpha coverage. */
	FVector4f AlphaCoverageThresholds;
	/** Use newer & faster mip generation filter */
	bool bUseNewMipFilter;
	/** Normalize normals after mip gen, before compression */
	bool bNormalizeNormals;
	/** The desired amount of mip sharpening. */
	float MipSharpening;
	/** For angular filtered cubemaps, the mip level which contains convolution with the diffuse cosine lobe. */
	uint32 DiffuseConvolveMipLevel;
	/** The size of the kernel with which mips should be sharpened. 2 for 2x2, 4 for 4x4, 6 for 6x6, 8 for 8x8 */
	uint32 SharpenMipKernelSize;
	/** For maximum resolution. */
	uint32 MaxTextureResolution;
	/** Format of the compressed texture, used to choose a compression DLL. */
	FName TextureFormatName;
	/** Whether the texture being built contains HDR source data */
	bool bHDRSource;
	/** Mipmap generation settings. */
	uint8 MipGenSettings; // TextureMipGenSettings, opaque to avoid dependencies on engine headers.
	/** Whether the texture being built is a cubemap. */
	uint32 bCubemap : 1;
	/** Whether the texture being built is a texture array. */
	uint32 bTextureArray : 1;
	/** Whether the texture being built is a volume. */
	uint32 bVolume : 1;
	/** Whether the texture being built from long/lat source to cubemap. */
	uint32 bLongLatSource : 1;
	/** Whether the texture contains color data in the sRGB colorspace. */
	uint32 bSRGB : 1;
	/** Advanced source encoding of the image. UE::Color::EEncoding / ETextureSourceEncoding (same thing) */
	uint8 SourceEncodingOverride;
	/** Whether the texture has a defined source color space. */
	bool bHasColorSpaceDefinition;
	/** Red chromaticity coordinate of the source color space. */
	FVector2f RedChromaticityCoordinate;
	/** Green chromaticity coordinate of the source color space. */
	FVector2f GreenChromaticityCoordinate;
	/** Blue chromaticity coordinate of the source color space. */
	FVector2f BlueChromaticityCoordinate;
	/** White chromaticity coordinate of the source color space. */
	FVector2f WhiteChromaticityCoordinate;
	/** Chromatic adaption method applied if the source white point differs from the working color space white point. */
	uint8 ChromaticAdaptationMethod;
	/** Whether the texture should use the legacy gamma space for converting to sRGB */
	uint32 bUseLegacyGamma : 1;
	/** Whether the border of the image should be maintained during mipmap generation. */
	uint32 bPreserveBorder : 1;
	/** Whether we should discard the alpha channel even if it contains non-zero values in the pixel data. */
	uint32 bForceNoAlphaChannel : 1;
	/** Whether we should not discard the alpha channel when it contains 1 for the entire texture. */
	uint32 bForceAlphaChannel : 1;
	/** Whether bokeh alpha values should be computed for the texture. */
	uint32 bComputeBokehAlpha : 1;
	/** Whether the contents of the red channel should be replicated to all channels. */
	uint32 bReplicateRed : 1;
	/** Whether the contents of the alpha channel should be replicated to all channels. */
	uint32 bReplicateAlpha : 1;
	/** Whether each mip should use the downsampled-with-average result instead of the sharpened result. */
	uint32 bDownsampleWithAverage : 1;
	/** Whether sharpening should prevent color shifts. */
	uint32 bSharpenWithoutColorShift : 1;
	/** Whether the border color should be black. */
	uint32 bBorderColorBlack : 1;
	/** Whether the green channel should be flipped. Typically only done on normal maps. */
	uint32 bFlipGreenChannel : 1;
	/** Calculate and apply a scale for YCoCg textures. This calculates a scale for each 4x4 block, applies it to the red and green channels and stores the scale in the blue channel. */
	uint32 bApplyYCoCgBlockScale : 1;
	/** 1:apply mip sharpening/blurring kernel to top mip as well (at half the kernel size), 0:don't */
	uint32 bApplyKernelToTopMip : 1;
	/** 1: renormalizes the top mip (only useful for normal maps, prevents artists errors and adds quality) 0:don't */
	uint32 bRenormalizeTopMip : 1;
	/** If set, the texture will generate a tiny placeholder hw texture and save off a copy of the top mip for CPU access. */
	uint32 bCPUAccessible : 1;
	/** e.g. CTM_RoughnessFromNormalAlpha */
	uint8 CompositeTextureMode;	// ECompositeTextureMode, opaque to avoid dependencies on engine headers.
	/* default 1, high values result in a stronger effect */
	float CompositePower;
	/** The source texture's final LOD bias (i.e. includes LODGroup based biases). Generally this does not affect the built texture as the
	* mips are stripped during cooking, however for tiling some platforms require knowing the actual texture size that will be created. In this
	* case they need to know the LOD bias to compensate.
	*/
	uint32 LODBias;
	/** The source texture's final LOD bias (i.e. includes LODGroup based biases). This allows cinematic mips as well. */
	uint32 LODBiasWithCinematicMips;
	/** Can the texture be streamed. This is deprecated because it was used in a single place in a single 
	*	platform for something that handled an edge case that never happened. That code is removes so this is
	*	never touched other than saving it, and it'll be removed soon.
	*/
	uint32 bStreamable_Unused : 1;
	/** Is the texture streamed using the VT system */
	uint32 bVirtualStreamable : 1;
	/** Whether to chroma key the image, replacing any pixels that match ChromaKeyColor with transparent black */
	uint32 bChromaKeyTexture : 1;
	/** How to stretch or pad the texture to a power of 2 size (if necessary); ETexturePowerOfTwoSetting::Type, opaque to avoid dependencies on Engine headers. */
	uint8 PowerOfTwoMode;
	/** The color used to pad the texture out if it is resized due to PowerOfTwoMode */
	FColor PaddingColor;
	/** If set to true, texture padding will be performed using colors of the border pixels in order to improve quality of the generated mipmaps. */
	bool bPadWithBorderColor;
	/** Width of the resized texture when using "Resize To Specific Resolution" padding and resizing option. If set to zero, original width will be used. */
	int32 ResizeDuringBuildX;
	/** Width of the resized texture when using "Resize To Specific Resolution" padding and resizing option. If set to zero, original height will be used. */
	int32 ResizeDuringBuildY;
	/** The color that will be replaced with transparent black if chroma keying is enabled */
	FColor ChromaKeyColor;
	/** The threshold that components have to match for the texel to be considered equal to the ChromaKeyColor when chroma keying (<=, set to 0 to require a perfect exact match) */
	float ChromaKeyThreshold;
	/** The quality of the compression algorithm (min 0 - lowest quality, highest cook speed, 4 - highest quality, lowest cook speed)
	* only used by ASTC formats right now.
	*/
	int32 CompressionQuality;

	// ETextureLossyCompressionAmount - oodle resolves this to RDO lambda during fast/final resolution.
	int32 LossyCompressionAmount;

	// which version of Oodle Texture to encode with
	FName OodleTextureSdkVersion;

	// If NAME_None, this encodes with the first non-versioned astcenc (4.2.0).
	// This must not include "." since it's mixed in to the ddc key
	FName AstcEncVersion;

	/** If set to true, then Oodle encoder preserves 0 and 255 (0.0 and 1.0) values exactly in alpha channel for BC3/BC7 and in all channels for BC4/BC5. */
	bool bOodlePreserveExtremes;

	/** Encoding settings resolved from fast/final.
	* Enums aren't accessible from this module:
	* ETextureEncodeEffort, ETextureUniversalTiling. */	
	uint8 OodleRDO;
	uint8 OodleEncodeEffort;
	uint8 OodleUniversalTiling;
	bool bOodleUsesRDO;

	/** Values > 1.0 will scale down source texture. Ignored for textures with mips */
	float Downscale;
	/** ETextureDownscaleOptions */
	uint8 DownscaleOptions;
	/** TextureAddress, opaque to avoid dependencies on engine headers. How to address the texture (clamp, wrap, ...) for virtual textures this is baked into the build data, for regular textures this is ignored. */
	int32 VirtualAddressingModeX;
	int32 VirtualAddressingModeY;
	/** Size in pixels of virtual texture tile, not including border */
	int32 VirtualTextureTileSize;
	/** Size in pixels of border on virtual texture tile */
	int32 VirtualTextureBorderSize;

	// Which encode speed this build settings represents.
	// This is not sent to the build worker, it is used to
	// return what was done to the UI.
	// ETextureEncodeSpeed, either Final or Fast.
	uint8 RepresentsEncodeSpeedNoSend;

	// "TextureAddress" enum values : (TA_Wrap default)
	uint8 TextureAddressModeX = 0;
	uint8 TextureAddressModeY = 0;
	uint8 TextureAddressModeZ = 0;

	// If the target format is a tiled format and can leverage reusing the linear encoding, this is not nullptr.
	const ITextureTiler* Tiler = nullptr;
	
	// If the target format has a tiler we store it here independent of shared linear
	const ITextureTiler* TilerEvenIfNotSharedLinear = nullptr;

	// If shared linear is enabled _at all_ and this texture in involved with that _at all_ then we set
	// this so we can segregate the derived data keys.
	bool bAffectedBySharedLinearEncoding = false;

	// If we have a child format, this is the base format (i.e. will have the platform prefix removed). Otherwise equal
	// to TextureFormatName.
	FName BaseTextureFormatName;

	// Cached pointer to BaseTextureFormatName. Resolved in FinalizeBuildSettingsForLayer and ResolveBuildSettings
	const ITextureFormat* BaseTextureFormat = nullptr;
	
	// Whether bHasTransparentAlpha is valid.
	bool bKnowAlphaTransparency = false;

	// Only valid if bKnowAlphaTransparency is true. This is whether the resulting texture is expected to require
	// an alpha channel based on scanning the source mips and analyzing the build settings.
	bool bHasTransparentAlpha = false;

	// If true, after encoding and potentially tiling, convert the texture back to something a PC can render
	// so we can view the final encoding.
	bool bDecodeForPCUsage = false;

	static constexpr uint32 MaxTextureResolutionDefault = TNumericLimits<uint32>::Max();

	/** Default settings. */
	FTextureBuildSettings()
		: bDoScaleMipsForAlphaCoverage(false)
		, AlphaCoverageThresholds(0, 0, 0, 0)
		, bUseNewMipFilter(false)
		, bNormalizeNormals(false)
		, MipSharpening(0.0f)
		, DiffuseConvolveMipLevel(0)
		, SharpenMipKernelSize(2)
		, MaxTextureResolution(MaxTextureResolutionDefault)
		, bHDRSource(false)
		, MipGenSettings(1 /*TMGS_SimpleAverage*/)
		, bCubemap(false)
		, bTextureArray(false)
		, bVolume(false)
		, bLongLatSource(false)
		, bSRGB(false)
		, SourceEncodingOverride(0 /*UE::Color::EEncoding::None*/)
		, bHasColorSpaceDefinition(false)
		, RedChromaticityCoordinate(0, 0)
		, GreenChromaticityCoordinate(0, 0)
		, BlueChromaticityCoordinate(0, 0)
		, WhiteChromaticityCoordinate(0, 0)
		, ChromaticAdaptationMethod(1 /*UE::Color::DEFAULT_CHROMATIC_ADAPTATION_METHOD*/)
		, bUseLegacyGamma(false)
		, bPreserveBorder(false)
		, bForceNoAlphaChannel(false)
		, bForceAlphaChannel(false)
		, bComputeBokehAlpha(false)
		, bReplicateRed(false)
		, bReplicateAlpha(false)
		, bDownsampleWithAverage(false)
		, bSharpenWithoutColorShift(false)
		, bBorderColorBlack(false)
		, bFlipGreenChannel(false)
		, bApplyYCoCgBlockScale(false)
		, bApplyKernelToTopMip(false)
		, bRenormalizeTopMip(false)
		, bCPUAccessible(false)
		, CompositeTextureMode(0 /*CTM_Disabled*/)
		, CompositePower(1.0f)
		, LODBias(0)
		, LODBiasWithCinematicMips(0)
		, bStreamable_Unused(false)
		, bVirtualStreamable(false)
		, bChromaKeyTexture(false)
		, PowerOfTwoMode(0 /*ETexturePowerOfTwoSetting::None*/)
		, PaddingColor(FColor::Black)
		, bPadWithBorderColor(false)
		, ResizeDuringBuildX(0)
		, ResizeDuringBuildY(0)
		, ChromaKeyColor(FColorList::Magenta)
		, ChromaKeyThreshold(1.0f / 255.0f)
		, CompressionQuality(-1)
		, LossyCompressionAmount(0 /* TLCA_Default */)
		, OodleTextureSdkVersion() // FName() == NAME_None
		, bOodlePreserveExtremes(false)
		, OodleRDO(30)
		, OodleEncodeEffort(0 /* ETextureEncodeEffort::Default */)
		, OodleUniversalTiling(0 /* ETextureUniversalTiling::Disabled */)
		, bOodleUsesRDO(false)
		, Downscale(0.0)
		, DownscaleOptions(0)
		, VirtualAddressingModeX(0)
		, VirtualAddressingModeY(0)
		, VirtualTextureTileSize(0)
		, VirtualTextureBorderSize(0)
	{
	}

	// GammaSpace is TextureCompressorModule is always a Destination gamma
	// we get FImages as input with source gamma space already set on them

	FORCEINLINE EGammaSpace GetSourceGammaSpace() const
	{
		return bSRGB ? ( bUseLegacyGamma ? EGammaSpace::Pow22 : EGammaSpace::sRGB ) : EGammaSpace::Linear;
	}
	
	FORCEINLINE EGammaSpace GetDestGammaSpace() const
	{
		return bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear;
	}

	UE_DEPRECATED(5.1, "Use GetDestGammaSpace or GetSourceGammaSpace (probably Dest)")
	FORCEINLINE EGammaSpace GetGammaSpace() const
	{
		return GetDestGammaSpace();
	}

	// If there is a choice to be had between two formats: one with alpha and one without, this returns which
	// one the texture expects.
	bool GetTextureExpectsAlphaInPixelFormat(bool bTopMipAlphaDetected) const
	{
		// note the order of operations! ( ForceNo takes precedence )
		if (bForceNoAlphaChannel)
		{
			return false;
		}
		if (bForceAlphaChannel)
		{
			return true;
		}
		return bTopMipAlphaDetected;
	}

	// bUnknownSourceAlphaFallback is whether we should treat the source as having alpha, in cases where the source alpha is unknown
	void GetOutputAlphaFromKnownAlphaOrFallback(bool* bOutOutputHasAlpha, bool bUnknownSourceAlphaFallback = true) const
	{
		if ( bKnowAlphaTransparency )
		{
			*bOutOutputHasAlpha = bHasTransparentAlpha;
		}
		else
		{
			// fallback without known alpha
			*bOutOutputHasAlpha = GetTextureExpectsAlphaInPixelFormat(bUnknownSourceAlphaFallback);
		}
	}

	// returns false if we can't determine instead of passing through a fallback
	bool GetOutputAlphaFromKnownAlphaOrFail(bool* bOutOutputHasAlpha) const
	{
		if (bKnowAlphaTransparency)
		{
			*bOutOutputHasAlpha = bHasTransparentAlpha;
			return true;
		}

		if (bForceNoAlphaChannel)
		{
			*bOutOutputHasAlpha = false;
			return true;
		}
		if (bForceAlphaChannel)
		{
			*bOutOutputHasAlpha = true;
			return true;
		}
		return false;
	}


	/*
	* Convert the build settings to an actual texture description containing enough information to describe the texture
	* to hardware APIs.
	*/
	TEXTURECOMPRESSOR_API void GetEncodedTextureDescription(FEncodedTextureDescription* OutTextureDescription, const ITextureFormat* InTextureFormat, int32 InEncodedMip0SizeX, int32 InEncodedMip0SizeY, int32 InEncodedMip0NumSlices, int32 InMipCount, bool bInImageHasAlphaChannel) const;
	TEXTURECOMPRESSOR_API void GetEncodedTextureDescriptionWithPixelFormat(FEncodedTextureDescription* OutTextureDescription, EPixelFormat InEncodedPixelFormat, int32 InEncodedMip0SizeX, int32 InEncodedMip0SizeY, int32 InEncodedMip0NumSlices, int32 InMipCount) const;

	// As per GetEncodedTextureDescription but the mip sizes are pre encoding. Return false for same reasons as GetOutputMipInfo
	TEXTURECOMPRESSOR_API bool GetEncodedTextureDescriptionFromSourceMips(FEncodedTextureDescription* OutTextureDescription, const ITextureFormat* InTextureFormat, int32 InSourceMip0SizeX, int32 InSourceMip0SizeY, int32 InSourceMip0NumSlices, int32 InSourceMipCount, bool bInImageHasAlphaChannel) const;

	// Return the expected size and mip count for a texture processed with these build settings.
	// returns false on virtual textures - physical textures only!
	// returns false if there's an invalid build settings - should never happen - build settings should
	// never get generated such that this fails!
	TEXTURECOMPRESSOR_API bool GetOutputMipInfo(
		int32 InMip0SizeX, int32 InMip0SizeY, int32 InMip0NumSlices, int32 InExistingMipCount,
		int32& OutMip0SizeX, int32& OutMip0SizeY, int32& OutMip0NumSlices, int32& OutMipCount) const;

	/* Obtain the OpenColorIO library version, primarily used for DDC invalidation. */
	TEXTURECOMPRESSOR_API static uint32 GetOpenColorIOVersion();
};

/**
 * Texture compression module interface.
 */
class ITextureCompressorModule : public IModuleInterface
{
public:

	/**
	 * Builds a texture from source images.
	 * @param SourceMips - The input mips.
	 * @param BuildSettings - Build settings.
	 * @param DebugTexturePathName - The path name of the texture being built, for logging/filtering/dumping.
	 * @param OutCompressedMips - The compressed mips built by the compressor.
	 * @param OutNumMipsInTail - The number of mips that are joined into a single mip tail mip
	 * @param OutExtData - Extra data that the runtime may need
	 * @returns true on success
	 
	// SourceMips can be freed by this call
	 */
	virtual bool BuildTexture(
		TArray<struct FImage>& SourceMips,
		TArray<struct FImage>& AssociatedNormalSourceMips,
		const FTextureBuildSettings& BuildSettings,
		FStringView DebugTexturePathName,
		TArray<FCompressedImage2D>& OutTextureMips,
		uint32& OutNumMipsInTail,
		uint32& OutExtData,
		UE::TextureBuildUtilities::FTextureBuildMetadata* OutMetadata
		) = 0;

	
	/**
	 * Generate a full mip chain. The input mip chain must have one or more mips.
	 * @param Settings - Preprocess settings.
	 * @param BaseImage - An image that will serve as the source for the generation of the mip chain.
	 * @param OutMipChain - An array that will contain the resultant mip images. Generated mip levels are appended to the array.
	 * @param MipChainDepth - number of mip images to produce. Mips chain is finished when either a 1x1 mip is produced or 'MipChainDepth' images have been produced.
	 */
	TEXTURECOMPRESSOR_API static void GenerateMipChain(
		const FTextureBuildSettings& Settings,
		const FImage& BaseImage,
		TArray<FImage> &OutMipChain,
		uint32 MipChainDepth
		);

	/**
	* Given the channel min/max for the top mip of a texture's source, determine whether there will be any transparency
	* after image processing occurs, and thus the texture will need to be BC3 for the purposes of AutoDXT format selection.
	* 
	* @param bOutAlphaIsTransparent		*This is undetermined if the return is false!*. If true, we expect the image after
	*									processing to require an alpha channel.
	* @return							Whether the alpha channel can be determined. There are plenty of edge cases where 
	*									it's not feasible to determine the result prior to full processing, however these
	*									are rare in practice.
	*/
	TEXTURECOMPRESSOR_API static bool DetermineAlphaChannelTransparency(const FTextureBuildSettings& InBuildSettings, const FLinearColor& InChannelMin, const FLinearColor& InChannelMax, bool& bOutAlphaIsTransparent);

	/**
     * Adjusts the colors of the image using the specified settings
     *
     * @param	Image			Image to adjust
     * @param	InBuildSettings	Image build settings
     */
	TEXTURECOMPRESSOR_API static void AdjustImageColors(FImage& Image, const FTextureBuildSettings& InBuildSettings);

	/**
	 * Generates the base cubemap mip from a longitude-latitude 2D image.
	 * @param OutMip - The output mip.
	 * @param SrcImage - The source longlat image.
	 */
	UE_DEPRECATED(5.3, "GenerateBaseCubeMipFromLongitudeLatitude2D with FTextureBuildSettings should be used.")
	TEXTURECOMPRESSOR_API static void GenerateBaseCubeMipFromLongitudeLatitude2D(FImage* OutMip, const FImage& SrcImage, const uint32 MaxCubemapTextureResolution, uint8 SourceEncodingOverride = 0);

	/**
	 * Generates the base cubemap mip from a longitude-latitude 2D image.
	 * @param OutMip - The output mip.
	 * @param SrcImage - The source longlat image.
	 * @param InBuildSettings - Image build settings
	 */
	TEXTURECOMPRESSOR_API static void GenerateBaseCubeMipFromLongitudeLatitude2D(FImage* OutMip, const FImage& SrcImage, const FTextureBuildSettings& InBuildSettings);

	/**
	 * Generates angularly filtered mips.
	 * @param InOutMipChain - The mip chain to angularly filter.
	 * @param NumMips - The number of mips the chain should have.
	 * @param DiffuseConvolveMipLevel - The mip level that contains the diffuse convolution.
	 */
	TEXTURECOMPRESSOR_API static void GenerateAngularFilteredMips(TArray<FImage>& InOutMipChain, int32 NumMips, uint32 DiffuseConvolveMipLevel);

	/**
	* Returns the number of mips that the given texture will generate with the given build settings, as well as the size of the top mip.
	* Used for physical textures - not virtual textures.
	*/
	//UE_DEPRECATED(5.5, "Just call GetOutputMipInfo directly off of build settings")
	static FORCEINLINE int32 GetMipCountForBuildSettings(
		int32 InMip0SizeX, int32 InMip0SizeY, int32 InMip0NumSlices, 
		int32 InExistingMipCount, const FTextureBuildSettings& InBuildSettings, 
		int32& OutMip0SizeX, int32& OutMip0SizeY, int32& OutMip0NumSlices)
	{
		int32 OutMipCount = 0;
		bool bSucceeded = InBuildSettings.GetOutputMipInfo(InMip0SizeX, InMip0SizeY, InMip0NumSlices, InExistingMipCount, OutMip0SizeX, OutMip0SizeY, OutMip0NumSlices, OutMipCount);
		check(bSucceeded);
		return OutMipCount;
	}
};
