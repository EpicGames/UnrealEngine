// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureCompressor : ModuleRules
{
	public TextureCompressor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"Engine",
		});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ImageCore",
				"OpenColorIOWrapper",
				"TextureBuildUtilities",
				"TextureFormat",
			}
			);

		//AddEngineThirdPartyPrivateStaticDependencies(Target, "nvTextureTools");
	}
}
